// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	//MeshBuilder module is a editor module
	public class MeshBuilder : ModuleRules
	{
		public MeshBuilder(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"MeshUtilitiesCommon",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"RHI",
					"Core",
					"CoreUObject",
					"Engine",
					"RenderCore",
					"MeshDescription",
					"StaticMeshDescription",
					"SkeletalMeshDescription",
					"MeshReductionInterface",
					"RawMesh",
					"MeshUtilities",
					"ClothingSystemRuntimeNv",
					"MeshBoneReduction",
					"SkeletalMeshUtilitiesCommon",
					"MeshBuilderCommon",
				}
			);

			AddEngineThirdPartyPrivateStaticDependencies(Target, "QuadricMeshReduction");

			PrivateIncludePathModuleNames.Add("NaniteBuilder");
			DynamicallyLoadedModuleNames.Add("NaniteBuilder");
       }
	}
}
