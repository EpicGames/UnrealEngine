// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Components.h"
#include "StaticMeshResources.h"
#include "ThirdPartyBuildOptimizationHelper.h"

class BuildOptimizationHelper
{
public:
	/** Helper struct for building acceleration structures. */
	struct FIndexAndZ
	{
		float Z;
		int32 Index;

		/** Default constructor. */
		FIndexAndZ() {}

		/** Initialization constructor. */
		FIndexAndZ(int32 InIndex, FVector V)
		{
			Z = 0.30f * V.X + 0.33f * V.Y + 0.37f * V.Z;
			Index = InIndex;
		}
	};

	/** Sorting function for vertex Z/index pairs. */
	struct FCompareIndexAndZ
	{
		FORCEINLINE bool operator()(FIndexAndZ const& A, FIndexAndZ const& B) const { return A.Z < B.Z; }
	};

	struct FMeshEdge
	{
		int32	Vertices[2];
		int32	Faces[2];
	};

	/**
	* This helper class builds the edge list for a mesh. It uses a hash of vertex
	* positions to edges sharing that vertex to remove the n^2 searching of all
	* previously added edges. 
	*/
	class FMeshEdgeBuilder
	{
	protected:
		/**
		* The list of indices to build the edge data from
		*/
		const TArray<uint32>& Indices;
		/**
		* The array of verts for vertex position comparison
		*/
		const FConstMeshBuildVertexView& Vertices;
		/**
		* The array of edges to create
		*/
		TArray<FMeshEdge>& Edges;
		/**
		* List of edges that start with a given vertex
		*/
		TMultiMap<FVector, FMeshEdge*> VertexToEdgeList;

		/**
		* Searches the list of edges to see if this one matches an existing and
		* returns a pointer to it if it does
		*
		* @param Index1 the first index to check for
		* @param Index2 the second index to check for
		*
		* @return nullptr if no edge was found, otherwise the edge that was found
		*/
		inline FMeshEdge* FindOppositeEdge(int32 Index1, int32 Index2)
		{
			FMeshEdge* Edge = nullptr;
			TArray<FMeshEdge*> EdgeList;
			
			// Search the hash for a corresponding vertex
			VertexToEdgeList.MultiFind((FVector)Vertices.Position[Index2], EdgeList);
			
			// Now search through the array for a match or not
			for (int32 EdgeIndex = 0; EdgeIndex < EdgeList.Num() && Edge == nullptr; ++EdgeIndex)
			{
				FMeshEdge* OtherEdge = EdgeList[EdgeIndex];
				// See if this edge matches the passed in edge
				if (OtherEdge != nullptr && DoesEdgeMatch(Index1, Index2, OtherEdge))
				{
					// We have a match
					Edge = OtherEdge;
				}
			}
			return Edge;
		}

		/**
		* Updates an existing edge if found or adds the new edge to the list
		*
		* @param Index1 the first index in the edge
		* @param Index2 the second index in the edge
		* @param Triangle the triangle that this edge was found in
		*/
		inline void AddEdge(int32 Index1, int32 Index2, int32 Triangle)
		{
			// If this edge matches another then just fill the other triangle
			// otherwise add it
			FMeshEdge* OtherEdge = FindOppositeEdge(Index1, Index2);
			if (OtherEdge == nullptr)
			{
				// Add a new edge to the array
				int32 EdgeIndex = Edges.AddZeroed();
				Edges[EdgeIndex].Vertices[0] = Index1;
				Edges[EdgeIndex].Vertices[1] = Index2;
				Edges[EdgeIndex].Faces[0] = Triangle;
				Edges[EdgeIndex].Faces[1] = INDEX_NONE;
				// Also add this edge to the hash for faster searches
				// NOTE: This relies on the array never being realloced!
				VertexToEdgeList.Add((FVector)Vertices.Position[Index1], &Edges[EdgeIndex]);
			}
			else
			{
				OtherEdge->Faces[1] = Triangle;
			}
		}

	public:
		/**
		* Initializes the values for the code that will build the mesh edge list
		*/
		FMeshEdgeBuilder(
			const TArray<uint32>& InIndices,
			const FConstMeshBuildVertexView& InVertices,
			TArray<FMeshEdge>& OutEdges
		) : Indices(InIndices), Vertices(InVertices), Edges(OutEdges)
		{
			// Presize the array so that there are no extra copies being done
			// when adding edges to it
			Edges.Empty(Indices.Num());
		}

		/**
		* Virtual dtor
		*/
		virtual ~FMeshEdgeBuilder() = default;

		/**
		* Uses a hash of indices to edge lists so that it can avoid the n^2 search
		* through the full edge list
		*/
		void FindEdges(void)
		{
			// @todo Handle something other than trilists when building edges
			int32 TriangleCount = Indices.Num() / 3;
			int32 EdgeCount = 0;
			// Work through all triangles building the edges
			for (int32 Triangle = 0; Triangle < TriangleCount; Triangle++)
			{
				// Determine the starting index
				int32 TriangleIndex = Triangle * 3;
				// Get the indices for the triangle
				int32 Index1 = Indices[TriangleIndex];
				int32 Index2 = Indices[TriangleIndex + 1];
				int32 Index3 = Indices[TriangleIndex + 2];
				// Add the first to second edge
				AddEdge(Index1, Index2, Triangle);
				// Now add the second to third
				AddEdge(Index2, Index3, Triangle);
				// Add the third to first edge
				AddEdge(Index3, Index1, Triangle);
			}
		}

		/**
		* This function determines whether a given edge matches or not for a static mesh
		*
		* @param Index1 The first index of the edge being checked
		* @param Index2 The second index of the edge
		* @param OtherEdge The edge to compare. Was found via the map
		*
		* @return true if the edge is a match, false otherwise
		*/
		inline bool DoesEdgeMatch(int32 Index1, int32 Index2, FMeshEdge* OtherEdge)
		{
			return Vertices.Position[OtherEdge->Vertices[1]] == Vertices.Position[Index1] &&
				OtherEdge->Faces[1] == INDEX_NONE;
		}
	};

	static void BuildDepthOnlyIndexBuffer(
		TArray<uint32>& OutDepthIndices,
		const FConstMeshBuildVertexView& InVertices,
		const TArray<uint32>& InIndices,
		const TArrayView<FStaticMeshSection>& InSections
	)
	{
		int32 NumVertices = InVertices.Position.Num();
		if (InIndices.Num() <= 0 || NumVertices <= 0)
		{
			OutDepthIndices.Empty();
			return;
		}

		// Create a mapping of index -> first overlapping index to accelerate the construction of the shadow index buffer.
		TArray<FIndexAndZ> VertIndexAndZ;
		VertIndexAndZ.Empty(NumVertices);
		for (int32 VertIndex = 0; VertIndex < NumVertices; VertIndex++)
		{
			VertIndexAndZ.Emplace(VertIndex, FVector(InVertices.Position[VertIndex]));
		}
		VertIndexAndZ.Sort(FCompareIndexAndZ());

		// Setup the index map. 0xFFFFFFFF == not set.
		TArray<uint32> IndexMap;
		IndexMap.AddUninitialized(NumVertices);
		FMemory::Memset(IndexMap.GetData(), 0xFF, NumVertices * sizeof(uint32));

		// Search for duplicates, quickly!
		for (int32 i = 0; i < VertIndexAndZ.Num(); i++)
		{
			uint32 SrcIndex = VertIndexAndZ[i].Index;
			float Z = VertIndexAndZ[i].Z;
			IndexMap[SrcIndex] = FMath::Min(IndexMap[SrcIndex], SrcIndex);

			// Search forward since we add pairs both ways.
			for (int32 j = i + 1; j < VertIndexAndZ.Num(); j++)
			{
				if (FMath::Abs(VertIndexAndZ[j].Z - Z) > THRESH_POINTS_ARE_SAME * 4.01f)
					break; // can't be any more dups

				uint32 OtherIndex = VertIndexAndZ[j].Index;
				if (InVertices.Position[SrcIndex].Equals(InVertices.Position[OtherIndex], 0.0f))
				{
					IndexMap[SrcIndex] = FMath::Min(IndexMap[SrcIndex], OtherIndex);
					IndexMap[OtherIndex] = FMath::Min(IndexMap[OtherIndex], SrcIndex);
				}
			}
		}

		// Build the depth-only index buffer by remapping all indices to the first overlapping
		// vertex in the vertex buffer.
		OutDepthIndices.Empty();
		for (int32 SectionIndex = 0; SectionIndex < InSections.Num(); ++SectionIndex)
		{
			const FStaticMeshSection& Section = InSections[SectionIndex];
			int32 FirstIndex = Section.FirstIndex;
			int32 LastIndex = FirstIndex + Section.NumTriangles * 3;
			for (int32 SrcIndex = FirstIndex; SrcIndex < LastIndex; ++SrcIndex)
			{
				uint32 VertIndex = InIndices[SrcIndex];
				OutDepthIndices.Add(IndexMap[VertIndex]);
			}
		}
	}

	static void CacheOptimizeVertexAndIndexBuffer(
		FMeshBuildVertexData& Vertices,
		TArray<TArray<uint32> >& PerSectionIndices,
		TArray<int32>& WedgeMap
	)
	{
		if (Vertices.Position.Num() <= 0)
		{
			return;
		}

		// Copy the vertices since we will be reordering them
		FMeshBuildVertexData OriginalVertices;
		OriginalVertices.Position = Vertices.Position;
		OriginalVertices.TangentX = Vertices.TangentX;
		OriginalVertices.TangentY = Vertices.TangentY;
		OriginalVertices.TangentZ = Vertices.TangentZ;
		OriginalVertices.CurveRadius = Vertices.CurveRadius;
		OriginalVertices.Color = Vertices.Color;
		OriginalVertices.UVs = Vertices.UVs;

		// Initialize a cache that stores which indices have been assigned
		TArray<int32> IndexCache;
		IndexCache.AddUninitialized(Vertices.Position.Num());
		FMemory::Memset(IndexCache.GetData(), INDEX_NONE, IndexCache.Num() * IndexCache.GetTypeSize());
		int32 NextAvailableIndex = 0;

		// Iterate through the section index buffers, 
		// Optimizing index order for the post transform cache (minimizes the number of vertices transformed), 
		// And vertex order for the pre transform cache (minimizes the amount of vertex data fetched by the GPU).
		for (int32 SectionIndex = 0; SectionIndex < PerSectionIndices.Num(); SectionIndex++)
		{
			TArray<uint32>& Indices = PerSectionIndices[SectionIndex];

			if (Indices.Num())
			{
				// Optimize the index buffer for the post transform cache with.
				BuildOptimizationThirdParty::CacheOptimizeIndexBuffer(Indices);

				// Copy the index buffer since we will be reordering it
				TArray<uint32> OriginalIndices = Indices;

				// Go through the indices and assign them new values that are coherent where possible
				for (int32 Index = 0; Index < Indices.Num(); Index++)
				{
					const int32 CachedIndex = IndexCache[OriginalIndices[Index]];

					if (CachedIndex == INDEX_NONE)
					{
						// No new index has been allocated for this existing index, assign a new one
						Indices[Index] = NextAvailableIndex;
						// Mark what this index has been assigned to
						IndexCache[OriginalIndices[Index]] = NextAvailableIndex;
						NextAvailableIndex++;
					}
					else
					{
						// Reuse an existing index assignment
						Indices[Index] = CachedIndex;
					}

					// Reorder the vertices based on the new index assignment
					Vertices.Position[Indices[Index]] = OriginalVertices.Position[OriginalIndices[Index]];
					Vertices.TangentX[Indices[Index]] = OriginalVertices.TangentX[OriginalIndices[Index]];
					Vertices.TangentY[Indices[Index]] = OriginalVertices.TangentY[OriginalIndices[Index]];
					Vertices.TangentZ[Indices[Index]] = OriginalVertices.TangentZ[OriginalIndices[Index]];
					if (Vertices.CurveRadius.Num() > 0)
					{
						Vertices.CurveRadius[Indices[Index]] = OriginalVertices.CurveRadius[OriginalIndices[Index]];
					}
					if (Vertices.Color.Num() > 0)
					{
						Vertices.Color[Indices[Index]] = OriginalVertices.Color[OriginalIndices[Index]];
					}
					for (int32 TexCoord = 0; TexCoord < Vertices.UVs.Num(); ++TexCoord)
					{
						Vertices.UVs[TexCoord][Indices[Index]] = OriginalVertices.UVs[TexCoord][OriginalIndices[Index]];
					}
				}
			}
		}

		for (int32 WedgeIndex = 0; WedgeIndex < WedgeMap.Num(); ++WedgeIndex)
		{
			const int32 MappedIndex = WedgeMap[WedgeIndex];
			if (MappedIndex != INDEX_NONE)
			{
				WedgeMap[WedgeIndex] = IndexCache[MappedIndex];
			}
		}
	}
};
