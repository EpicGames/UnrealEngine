// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/PlatformTime.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMeshDescriptionBuildStatistic, Log, All);

namespace BuildStatisticManager
{
	struct FStatisticData
	{
	public:
		FStatisticData()
			: TotalTime(0.0)
			, Counter(0)
		{ }
		double TotalTime;
		uint32 Counter;
	};

	class FBuildStatisticScope
	{
	public:
		FBuildStatisticScope(const FString& InTimerDescription, FStatisticData& InStatisticData)
			: TimerDescription(InTimerDescription)
			, StatisticData(InStatisticData)
		{
			StartTime = FPlatformTime::Cycles();
		}
		~FBuildStatisticScope()
		{
			double ScopeTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles() - StartTime);
			StatisticData.TotalTime += ScopeTime;
			StatisticData.Counter += 1;
			UE_LOGF(LogMeshDescriptionBuildStatistic, Log, "%ls: %f seconds - Counter: %d - Total Time: %f", *TimerDescription, ScopeTime, StatisticData.Counter, StatisticData.TotalTime);
		}
	private:
		FString TimerDescription;
		int32 StartTime;
		FStatisticData& StatisticData;
	};
}