// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshBuilder.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"


FMeshBuilder::FMeshBuilder()
{

}

//
// DEPRECATED METHODS & DEPENDENCIES
//

#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"

bool FMeshBuilder::Build(
	FStaticMeshRenderData& OutRenderData,
	UStaticMesh* StaticMesh,
	const FStaticMeshLODGroup& LODGroup,
	bool bAllowNanite)
{
	return Build(OutRenderData, FStaticMeshBuildParameters(StaticMesh, nullptr, LODGroup));
}

bool FMeshBuilder::Build(const FSkeletalMeshBuildParameters& SkeletalMeshBuildParameters)
{
	FSkeletalMeshRenderData* RenderData = SkeletalMeshBuildParameters.SkeletalMesh->GetResourceForRendering();
	return Build(*RenderData, SkeletalMeshBuildParameters);
}