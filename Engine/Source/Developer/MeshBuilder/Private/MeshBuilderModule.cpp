// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMeshBuilderModule.h"
#include "Modules/ModuleManager.h"

#include "StaticMeshBuilder.h"
#include "Engine/StaticMesh.h"
#include "SkeletalMeshBuilder.h"
#include "Engine/SkeletalMesh.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY_STATIC(LogMeshBuilderModule, Display, All);

class FMeshBuilderModule : public IMeshBuilderModule
{
public:

	FMeshBuilderModule()
	{
	}

	virtual void StartupModule() override
	{
		// Register any modular features here
	}

	virtual void ShutdownModule() override
	{
		// Unregister any modular features here
	}

	virtual bool BuildMesh(class FStaticMeshRenderData& OutRenderData, const FStaticMeshBuildParameters& BuildParameters) override;

	virtual bool BuildMeshVertexPositions(
		UObject* StaticMesh,
		TArray<uint32>& Indices,
		TArray<FVector3f>& Vertices,
		FStaticMeshSectionArray& Sections) override;

	virtual bool BuildSkeletalMesh(FSkeletalMeshRenderData& OutRenderData, const FSkeletalMeshBuildParameters& SkeletalMeshBuildParameters) override;

private:

};

IMPLEMENT_MODULE(FMeshBuilderModule, MeshBuilder );

bool FMeshBuilderModule::BuildMesh(FStaticMeshRenderData& OutRenderData, const FStaticMeshBuildParameters& BuildParameters)
{
	//Call the static mesh builder
	return FStaticMeshBuilder().Build(OutRenderData, BuildParameters);
}

bool FMeshBuilderModule::BuildMeshVertexPositions(
	UObject* Mesh,
	TArray<uint32>& Indices,
	TArray<FVector3f>& Vertices,
	FStaticMeshSectionArray& Sections)
{
	UStaticMesh* StaticMesh = Cast< UStaticMesh >(Mesh);
	if (StaticMesh)
	{
		//Call the static mesh builder
		return FStaticMeshBuilder().BuildMeshVertexPositions(StaticMesh, Indices, Vertices, Sections);
	}

	// Add some log lines to help identify why BuildMeshVertexPositions has failed.
	if ( Mesh==nullptr )
	{
		UE_LOGF(LogMeshBuilderModule, Warning, "NULL Mesh UObject passed to BuildMeshVertexPositions");
	}	
	else if (StaticMesh == nullptr)
	{
		UE_LOGF(LogMeshBuilderModule, Warning, "Mesh UObject %ls (%ls) passed to BuildMeshVertexPositions could not be converted to a UStaticMesh", *Mesh->GetFName().ToString(), *Mesh->GetPathName());
	}

	return false;
}

bool FMeshBuilderModule::BuildSkeletalMesh(FSkeletalMeshRenderData& OutRenderData, const FSkeletalMeshBuildParameters& SkeletalMeshBuildParameters)
{
	//Call the skeletal mesh builder
	return FSkeletalMeshBuilder().Build(OutRenderData, SkeletalMeshBuildParameters);
}