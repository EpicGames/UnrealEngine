// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshDescriptionHelper.h"

#include "BuildStatisticManager.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Engine/EngineTypes.h"
#include "Engine/StaticMesh.h"
#include "IMeshReductionInterfaces.h"
#include "IMeshReductionManagerModule.h"
#include "Materials/Material.h"
#include "Modules/ModuleManager.h"
#include "RawMesh.h"
#include "Tasks/Task.h"
#include "RenderUtils.h"
#include "StaticMeshAttributes.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "StaticMeshOperations.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

//Enable all check
//#define ENABLE_NTB_CHECK

DEFINE_LOG_CATEGORY(LogMeshDescriptionBuildStatistic);

FMeshDescriptionHelper::FMeshDescriptionHelper(FMeshBuildSettings* InBuildSettings)
	: BuildSettings(InBuildSettings)
{
}

void FMeshDescriptionHelper::SetupRenderMeshDescription(UObject* Owner, FMeshDescription& RenderMeshDescription, bool bForNanite, bool bNeedTangents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshDescriptionHelper::SetupRenderMeshDescription);

	UStaticMesh* StaticMesh = Cast<UStaticMesh>(Owner);
	check(StaticMesh);

	float ComparisonThreshold = (BuildSettings->bRemoveDegenerates && !bForNanite) ? THRESH_POINTS_ARE_SAME : 0.0f;
	
	// Compact the mesh description prior to performing operations
	if (RenderMeshDescription.NeedsCompact())
	{
		FElementIDRemappings Remappings;
		RenderMeshDescription.Compact(Remappings);
	}

	//Make sure we do not have nan or infinite float in the mesh description data
	FStaticMeshOperations::ValidateAndFixData(RenderMeshDescription, Owner->GetName());

	//This function make sure the Polygon Normals Tangents Binormals are computed and also remove degenerated triangle from the render mesh description.
	FStaticMeshOperations::ComputeTriangleTangentsAndNormals(RenderMeshDescription, ComparisonThreshold, *Owner->GetPathName());

	FVertexInstanceArray& VertexInstanceArray = RenderMeshDescription.VertexInstances();

	FStaticMeshAttributes Attributes(RenderMeshDescription);
	TVertexInstanceAttributesRef<FVector3f> Normals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> Tangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> BinormalSigns = Attributes.GetVertexInstanceBinormalSigns();

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return;
	}

	// Find overlapping corners to accelerate adjacency.
	FStaticMeshOperations::FindOverlappingCorners(OverlappingCorners, RenderMeshDescription, ComparisonThreshold);

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return;
	}

	// Static meshes always blend normals of overlapping corners.
	EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::BlendOverlappingNormals;
	ComputeNTBsOptions |= BuildSettings->bComputeWeightedNormals ? EComputeNTBsFlags::WeightedNTBs : EComputeNTBsFlags::None;
	ComputeNTBsOptions |= BuildSettings->bRecomputeNormals ? EComputeNTBsFlags::Normals : EComputeNTBsFlags::None;
	ComputeNTBsOptions |= BuildSettings->bUseMikkTSpace ? EComputeNTBsFlags::UseMikkTSpace : EComputeNTBsFlags::None;

	// Set extra options for non-Nanite meshes
	if (!bForNanite)
	{
		ComputeNTBsOptions |= BuildSettings->bRemoveDegenerates ? EComputeNTBsFlags::IgnoreDegenerateTriangles : EComputeNTBsFlags::None;
	}

	if (bNeedTangents)
	{
		ComputeNTBsOptions |= BuildSettings->bRecomputeTangents ? EComputeNTBsFlags::Tangents : EComputeNTBsFlags::None;
	}

	// Compute any missing normals or tangents.
	FStaticMeshOperations::ComputeTangentsAndNormals(RenderMeshDescription, ComputeNTBsOptions);

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return;
	}

	if (BuildSettings->bGenerateLightmapUVs && VertexInstanceArray.Num() > 0)
	{
		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
		int32 NumIndices = VertexInstanceUVs.GetNumChannels();
		//Verify the src light map channel
		if (BuildSettings->SrcLightmapIndex >= NumIndices)
		{
			BuildSettings->SrcLightmapIndex = 0;
		}
		//Verify the destination light map channel
		if (BuildSettings->DstLightmapIndex >= NumIndices)
		{
			//Make sure we do not add illegal UV Channel index
			if (BuildSettings->DstLightmapIndex >= MAX_MESH_TEXTURE_COORDS_MD)
			{
				BuildSettings->DstLightmapIndex = MAX_MESH_TEXTURE_COORDS_MD - 1;
			}

			//Add some unused UVChannel to the mesh description for the lightmapUVs
			VertexInstanceUVs.SetNumChannels(BuildSettings->DstLightmapIndex + 1);
			BuildSettings->DstLightmapIndex = NumIndices;
		}
		FStaticMeshOperations::CreateLightMapUVLayout(RenderMeshDescription,
			BuildSettings->SrcLightmapIndex,
			BuildSettings->DstLightmapIndex,
			BuildSettings->MinLightmapResolution,
			(ELightmapUVVersion)StaticMesh->GetLightmapUVVersion(),
			OverlappingCorners);
	}
}

void FMeshDescriptionHelper::ReduceLOD(const FMeshDescription& BaseMesh, FMeshDescription& DestMesh, const FMeshReductionSettings& ReductionSettings, const FOverlappingCorners& InOverlappingCorners, float &OutMaxDeviation)
{
	IMeshReductionManagerModule& MeshReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	IMeshReduction* MeshReduction = MeshReductionModule.GetStaticMeshReductionInterface();

	
	if (!MeshReduction)
	{
		// no reduction possible
		OutMaxDeviation = 0.f;
		return;
	}

	OutMaxDeviation = ReductionSettings.MaxDeviation;
	MeshReduction->ReduceMeshDescription(DestMesh, OutMaxDeviation, BaseMesh, InOverlappingCorners, ReductionSettings);
}

void FMeshDescriptionHelper::FindOverlappingCorners(const FMeshDescription& MeshDescription, float ComparisonThreshold)
{
	FStaticMeshOperations::FindOverlappingCorners(OverlappingCorners, MeshDescription, ComparisonThreshold);
}

