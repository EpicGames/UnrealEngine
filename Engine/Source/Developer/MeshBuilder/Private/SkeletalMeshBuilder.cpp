// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshBuilder.h"
#include "Modules/ModuleManager.h"
#include "MeshBoneReduction.h"
#include "Engine/EngineTypes.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "PhysicsEngine/BodySetup.h"
#include "MeshDescription.h"
#include "MeshAttributes.h"
#include "MeshDescriptionHelper.h"
#include "MeshBuild.h"
#include "SkeletalMeshAttributes.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "GPUSkinVertexFactory.h"
#include "ThirdPartyBuildOptimizationHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "LODUtilities.h"
#include "ClothingAsset.h"
#include "MeshUtilities.h"
#include "EditorFramework/AssetImportData.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"
#include "NaniteBuilder.h"
#include "NaniteHelper.h"
#include "Rendering/NaniteResources.h"

DEFINE_LOG_CATEGORY(LogSkeletalMeshBuilder);

static void BuildNaniteFallbackMeshDescription(
	const USkeletalMesh* InSkeletalMesh,
	const Nanite::IBuilderModule::FOutputMeshData& InMeshData,
	FMeshDescription& OutMesh,
	const FSkeletalMeshUnifiedBoneIndices& UnifiedBoneIndices
)
{
	using UE::AnimationCore::FBoneWeights;

	OutMesh.Empty();

	FSkeletalMeshAttributes Attributes(OutMesh);
	Attributes.Register();

	const TArray<FSkinWeightProfileInfo>& SkinWeightProfiles = InSkeletalMesh->GetSkinWeightProfiles();
	for (const FSkinWeightProfileInfo& SkinWeightProfileInfo : SkinWeightProfiles)
	{
		Attributes.RegisterSkinWeightAttribute(SkinWeightProfileInfo.Name);
	}

	const FReferenceSkeleton& RefSkeleton = InSkeletalMesh->GetRefSkeleton();
	const int NumBones = RefSkeleton.GetRawBoneNum();

	const int32 NumVertices = InMeshData.Vertices.Position.Num();
	const int32 NumUVChannels = InMeshData.Vertices.UVs.Num();
	const int32 NumTriangles = InMeshData.TriangleIndices.Num() / 3;
	const int32 NumPolyGroups = InMeshData.Sections.Num();

	OutMesh.ReserveNewVertices(NumVertices);
	OutMesh.ReserveNewVertexInstances(NumVertices);
	OutMesh.ReserveNewTriangles(NumTriangles);
	OutMesh.ReserveNewPolygonGroups(NumPolyGroups);

	OutMesh.SetNumUVChannels(NumUVChannels);
	OutMesh.VertexInstanceAttributes().SetAttributeChannelCount(MeshAttribute::VertexInstance::TextureCoordinate, NumUVChannels);
	for (int32 UVChannelIndex = 0; UVChannelIndex < NumUVChannels; ++UVChannelIndex)
	{
		OutMesh.ReserveNewUVs(NumVertices, UVChannelIndex);
	}

	Attributes.ReserveNewBones(NumBones);

	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	FSkinWeightsVertexAttributesRef VertexSkinWeights = Attributes.GetVertexSkinWeights();

	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

	TPolygonGroupAttributesRef<FName> PolygonGroupMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();

	FSkeletalMeshAttributes::FBoneNameAttributesRef BoneNames = Attributes.GetBoneNames();
	FSkeletalMeshAttributes::FBoneParentIndexAttributesRef BoneParentIndices = Attributes.GetBoneParentIndices();
	FSkeletalMeshAttributes::FBonePoseAttributesRef BonePoses = Attributes.GetBonePoses();

	for (int32 InVertIndex = 0; InVertIndex < NumVertices; ++InVertIndex)
	{
		const FVertexID VertexID(InVertIndex);
		const FVertexInstanceID VertexInstanceID(InVertIndex);

		// TODO: Deduplicate vertex positions?
		OutMesh.CreateVertexWithID(VertexID);
		OutMesh.CreateVertexInstanceWithID(VertexInstanceID, VertexID);

		FVector3f Position = InMeshData.Vertices.Position[InVertIndex];
		FVector3f TangentX = InMeshData.Vertices.TangentX[InVertIndex];
		FVector3f TangentY = InMeshData.Vertices.TangentY[InVertIndex];
		FVector3f TangentZ = InMeshData.Vertices.TangentZ[InVertIndex];

		const uint32 NumBoneInfluences = InMeshData.Vertices.BoneIndices.Num();
		check(NumBoneInfluences == InMeshData.Vertices.BoneWeights.Num() && NumBoneInfluences <= MAX_TOTAL_INFLUENCES);
		
		const float BinormalSign = GetBasisDeterminantSign(FVector(TangentX), FVector(TangentY), FVector(TangentZ));
		const FColor Color = InMeshData.Vertices.Color.IsValidIndex(InVertIndex) ?
			InMeshData.Vertices.Color[InVertIndex] : FColor::White;

		VertexPositions.Set(VertexID, Position);
		VertexInstanceNormals.Set(VertexInstanceID, TangentZ);
		VertexInstanceTangents.Set(VertexInstanceID, TangentX);
		VertexInstanceBinormalSigns.Set(VertexInstanceID, BinormalSign);
		VertexInstanceColors.Set(VertexInstanceID, FVector4f(FLinearColor(Color)));

		for (int32 UVChannelIndex = 0; UVChannelIndex < NumUVChannels; ++UVChannelIndex)
		{
			const FVector2f UV = InMeshData.Vertices.UVs[UVChannelIndex][InVertIndex];
			VertexInstanceUVs.Set(VertexInstanceID, UVChannelIndex, UV);
		}


		FBoneIndexType InfluenceBones[MAX_TOTAL_INFLUENCES] = {};
		uint16 InfluenceWeights[MAX_TOTAL_INFLUENCES] = {};

		for (uint32 InfluenceIndex = 0; InfluenceIndex < NumBoneInfluences; ++InfluenceIndex)
		{
			const TArray<uint16>& BoneIndices = InMeshData.Vertices.BoneIndices[InfluenceIndex];
			const TArray<uint16>& BoneWeights = InMeshData.Vertices.BoneWeights[InfluenceIndex];

			uint16 BoneIndex = BoneIndices[InVertIndex];

			if (UnifiedBoneIndices.IsValid())
			{
				// Input vertices are in unified bone space, but the fallback path expects bones
				// in skeleton space which it will then remap to unified in its subsequent build step.
				BoneIndex = UnifiedBoneIndices.GetUnifiedToSkeletonIndex(BoneIndex);
			}

			InfluenceBones[InfluenceIndex] = BoneIndex;
			InfluenceWeights[InfluenceIndex] = BoneWeights[InVertIndex];
		}

		VertexSkinWeights.Set(VertexID, FBoneWeights::Create(InfluenceBones, InfluenceWeights));
	}

	const TArray<FSkeletalMaterial>& Materials = InSkeletalMesh->GetMaterials();
	for (const Nanite::FMeshDataSection& Section : InMeshData.Sections)
	{
		const FPolygonGroupID PolygonGroupID = OutMesh.CreatePolygonGroup();
		const FName MaterialSlotName = Materials.IsValidIndex(Section.MaterialIndex) ?
			Materials[Section.MaterialIndex].ImportedMaterialSlotName : NAME_None;
		PolygonGroupMaterialSlotNames.Set(PolygonGroupID, MaterialSlotName);

		for (uint32 TriIndex = 0; TriIndex < Section.NumTriangles; ++TriIndex)
		{
			const FVertexInstanceID TriVertInstanceIDs[] = {
				FVertexInstanceID(InMeshData.TriangleIndices[Section.FirstIndex + TriIndex * 3 + 0]),
				FVertexInstanceID(InMeshData.TriangleIndices[Section.FirstIndex + TriIndex * 3 + 1]),
				FVertexInstanceID(InMeshData.TriangleIndices[Section.FirstIndex + TriIndex * 3 + 2])
			};

			OutMesh.CreateTriangle(PolygonGroupID, MakeConstArrayView(TriVertInstanceIDs, 3));
		}
	}

	// Set Bone Attributes
	for (int Index = 0; Index < NumBones; ++Index)
	{
		const FMeshBoneInfo& BoneInfo = RefSkeleton.GetRawRefBoneInfo()[Index];
		const FTransform& BoneTransform = RefSkeleton.GetRawRefBonePose()[Index];

		const FBoneID BoneID = Attributes.CreateBone();

		BoneNames.Set(BoneID, BoneInfo.Name);
		BoneParentIndices.Set(BoneID, BoneInfo.ParentIndex);
		BonePoses.Set(BoneID, BoneTransform);
	}
}

namespace SkeletalMeshBuilderPrivate
{

/** Context data for a skeletal mesh build */
class FContext
{
public:
	USkeletalMesh*						SkeletalMesh			= nullptr;
	IMeshUtilities*						MeshUtilities			= nullptr;
	Nanite::IBuilderModule*				NaniteBuilder			= nullptr;
	const FSkeletalMeshLODInfo*			LODInfo					= nullptr;
	const FMeshDescription* 			SourceMeshDescription	= nullptr;
	FMeshDescription*					FallbackMeshDescription	= nullptr;
	const ITargetPlatform*				TargetPlatform			= nullptr;
	FSkeletalMeshImportData 			ImportData;
	IMeshUtilities::MeshBuildOptions	Options;
	Nanite::FInputAssemblyData			NaniteAssemblyData;
	FMeshNaniteSettings 				NaniteSettings;
	TUniquePtr<FSkeletalMeshLODModel>	LODModelClone;
	int32 								LODIndex				= INDEX_NONE;
	bool								bNaniteAssembly 		= false;
	bool								bNaniteAssemblyPart		= false;
	bool								bRegenDepLODs			= false;
	bool								bBuildNaniteFallback	= false;
	Nanite::FMeshDataSectionArray		NaniteFallbackMeshSections;
	const FSkeletalMeshUnifiedBoneIndices UnifiedBoneIndices;

	FContext(
		const FSkeletalMeshBuildParameters& InBuildParameters,
		const FMeshDescription* InSourceMeshDescription,
		FMeshDescription* OutFallbackMeshDescription,
		bool bInBuildNanite,
		bool bInBuildNaniteFallback,
		Nanite::FMeshDataSectionArray&& InNaniteFallbackMeshSections = {},
		TConstArrayView<uint16> UnifiedBoneMap = {}
	)
		: SkeletalMesh(InBuildParameters.SkeletalMesh)
		, MeshUtilities(&FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities"))
		, SourceMeshDescription(InSourceMeshDescription)
		, FallbackMeshDescription(OutFallbackMeshDescription)
		, TargetPlatform(InBuildParameters.TargetPlatform)
		, LODIndex(InBuildParameters.LODIndex)
		, bRegenDepLODs(InBuildParameters.bRegenDepLODs)
		, bBuildNaniteFallback(bInBuildNaniteFallback)
		, NaniteFallbackMeshSections(InNaniteFallbackMeshSections)
		, UnifiedBoneIndices(UnifiedBoneMap)
	{
		if (bInBuildNanite && SkeletalMesh->IsNaniteEnabled())
		{
			NaniteBuilder = &Nanite::IBuilderModule::Get();
			bNaniteAssembly = SkeletalMesh->IsNaniteAssembly();
		}

		Init();
	}

	FContext(const FContext& ParentContext, USkeletalMesh& AssemblyPartMesh)
		: SkeletalMesh(&AssemblyPartMesh)
		, MeshUtilities(ParentContext.MeshUtilities)
		, NaniteBuilder(ParentContext.NaniteBuilder)
		, TargetPlatform(ParentContext.TargetPlatform)
		, LODIndex(0)
		, bNaniteAssemblyPart(true)
		, UnifiedBoneIndices(ParentContext.UnifiedBoneIndices)
	{
		SourceMeshDescription = AssemblyPartMesh.GetMeshDescription(LODIndex);
		Init(&ParentContext);
	}

	bool IsNaniteBuildEnabled() const		{ return NaniteBuilder != nullptr; }
	bool IsNaniteAssemblyBuild() const		{ return bNaniteAssembly; }
	bool IsNaniteAssemblyPartBuild() const	{ return bNaniteAssemblyPart; }
	
	FSkeletalMeshLODModel& GetLODModel()
	{
		if (LODModelClone.IsValid())
		{
			return *LODModelClone;
		}

		// NOTE: We don't cache this because some build steps (like reductions) might cause it to be reallocated
		return SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	}

	int32 GetNumTexCoords() const
	{
		//We need to send rendering at least one tex coord buffer
		return FMath::Max<int32>(1, ImportData.NumTexCoords);
	}

	void UnbindClothingAndBackup()
	{
		//We want to backup in case the LODModel is regenerated, this data is use to validate in the UI if the ddc must be rebuild
		BackupBuildStringID = SkeletalMesh->GetImportedModel()->LODModels[LODIndex].BuildStringID;
		FLODUtilities::UnbindClothingAndBackup(SkeletalMesh, ClothingBindings, LODIndex);
	}

	void BuildLODModel()
	{
		FSkeletalMeshLODModel& LODModel = GetLODModel();

		TArray<FVector3f> LODPoints;
		TArray<SkeletalMeshImportData::FMeshWedge> LODWedges;
		TArray<SkeletalMeshImportData::FMeshFace> LODFaces;
		TArray<SkeletalMeshImportData::FVertInfluence> LODInfluences;
		TArray<int32> LODPointToRawMap;
		ImportData.CopyLODImportData(LODPoints, LODWedges, LODFaces, LODInfluences, LODPointToRawMap);

		// BaseLOD needs to make sure the source data fit with the skeletal mesh materials array before using meshutilities.BuildSkeletalMesh
		FLODUtilities::AdjustImportDataFaceMaterialIndex(SkeletalMesh->GetMaterials(), ImportData.Materials, LODFaces, LODIndex);

		// Create skinning streams for NewModel.
		MeshUtilities->BuildSkeletalMesh(
			LODModel, 
			SkeletalMesh->GetPathName(),
			SkeletalMesh->GetRefSkeleton(),
			LODInfluences,
			LODWedges,
			LODFaces,
			LODPoints,
			LODPointToRawMap,
			Options
		);

		// Set texture coordinate count on the new model.
		LODModel.NumTexCoords = GetNumTexCoords();

		// Cache the vertex/triangle count in the InlineReductionCacheData so we can know if the LODModel need reduction or not.
		TArray<FInlineReductionCacheData>& InlineReductionCacheDatas = SkeletalMesh->GetImportedModel()->InlineReductionCacheDatas;
		if (!InlineReductionCacheDatas.IsValidIndex(LODIndex))
		{
			InlineReductionCacheDatas.AddDefaulted((LODIndex + 1) - InlineReductionCacheDatas.Num());
		}
		if (ensure(InlineReductionCacheDatas.IsValidIndex(LODIndex)))
		{
			InlineReductionCacheDatas[LODIndex].SetCacheGeometryInfo(LODModel);
		}

		// For Nanite fallbacks, it's possible that all triangles of a given mesh section were simplified out. Add empty mesh
		// sections for any unrepresented materials.
		for (const Nanite::FMeshDataSection& FallbackSection : NaniteFallbackMeshSections)
		{
			int32 MaxOriginalDataSectionIndex = -1;
			if (LODModel.Sections.FindByPredicate(
				[&MaxOriginalDataSectionIndex, &FallbackSection](const FSkelMeshSection& Section)
				{
					MaxOriginalDataSectionIndex = FMath::Max(MaxOriginalDataSectionIndex, Section.OriginalDataSectionIndex);
					return Section.MaterialIndex == FallbackSection.MaterialIndex;
				}) == nullptr)
			{
				// Add an empty mesh section for this material
				FSkelMeshSection& NewSection = LODModel.Sections.Emplace_GetRef();
				FMemory::Memzero(NewSection);

				NewSection.MaterialIndex				= FallbackSection.MaterialIndex;
				NewSection.bDisabled 					= true;
				NewSection.bCastShadow					= EnumHasAnyFlags(FallbackSection.Flags, Nanite::EMeshDataSectionFlags::CastShadow);
				NewSection.bVisibleInRayTracing			= EnumHasAnyFlags(FallbackSection.Flags, Nanite::EMeshDataSectionFlags::VisibleInRayTracing);
				NewSection.OriginalDataSectionIndex		= MaxOriginalDataSectionIndex + 1;
				NewSection.ChunkedParentSectionIndex	= INDEX_NONE;
				NewSection.CorrespondClothAssetIndex	= INDEX_NONE;
			}
		}

		// Re-Apply the user section changes, the UserSectionsData is map to original section and should match the built LODModel
		LODModel.SyncronizeUserSectionsDataArray();
	}

	bool BuildNaniteAssemblyData()
	{
		if (!NaniteAssembliesSupported())
		{
			UE_LOGF(LogSkeletalMesh, Warning,
				"Failed to build Nanite Assembly skeletal mesh %ls. Nanite assemblies are not enabled for the project. " "Enable Nanite Foliage in the project settings or configure r.Nanite.AllowAssemblies=1 to enable support.",
				*SkeletalMesh->GetPathName());
			return false;
		}
		if (!SkeletalMesh->HasCachedNaniteAssemblyReferences())
		{
			UE_LOGF(LogSkeletalMesh, Warning,
				"Failed to build Nanite Assembly skeletal mesh %ls. The referenced skeletal meshes were not cached before build.",
				*SkeletalMesh->GetPathName());
			return false;
		}

		// Get the assembly references that should have been resolved before build
		const FNaniteAssemblyData& InAssemblyData = NaniteSettings.NaniteAssemblyData;
		const TArray<TObjectPtr<USkeletalMesh>>& PartReferences = SkeletalMesh->GetCachedNaniteAssemblyReferences();
		check(PartReferences.Num() == InAssemblyData.Parts.Num());

		TMap<USkeletalMesh*, Nanite::FAssemblyPartResourceRef> ResourceLookup;
		ResourceLookup.Reserve(PartReferences.Num());

		for (int32 PartIndex = 0; PartIndex < InAssemblyData.Parts.Num(); ++PartIndex)
		{
			Nanite::FAssemblyPartResourceRef Resource;

			if (USkeletalMesh* PartMesh = PartReferences[PartIndex])
			{
				if (Nanite::FAssemblyPartResourceRef* ExistingResource = ResourceLookup.Find(PartMesh))
				{
					Resource = *ExistingResource;
				}
				else
				{
					FContext ChildContext(*this, *PartMesh);
					
					// HACK: We need this LOD model to be updated from source mesh description in case it's been reduced.
					// TODO: Eliminate the need for this by building Nanite input data from the import data instead
					ChildContext.BuildLODModel();
					
					Resource = ChildContext.BuildNaniteAssemblyPart();
					ResourceLookup.Add(PartMesh, Resource);
				}
			}

			if (!Resource.IsValid())
			{
				UE_LOGF(LogSkeletalMesh, Warning, 
					"Failed to build Nanite Assembly part from skeletal mesh (%ls). See previous line(s) for details.",
					*InAssemblyData.Parts[PartIndex].MeshObjectPath.GetAssetName());
				return false;
			}

			auto& OutPart = NaniteAssemblyData.Parts.AddDefaulted_GetRef();
			OutPart.Resource = Resource;
		
			// Apply the material remap
			const TArray<int32>& MaterialRemap = InAssemblyData.Parts[PartIndex].MaterialRemap;
			for (int32 i = 0; i < Nanite::MaxSectionArraySize; ++i)
			{
				if (MaterialRemap.Num() == 0)
				{
					// No remaps = match indices
					OutPart.MaterialRemap[i] = i;
				}
				else if (MaterialRemap.IsValidIndex(i))
				{
					OutPart.MaterialRemap[i] = MaterialRemap[i];
				}
				else
				{
					// Index is unrepresented in the remap (may not be a valid index). Fallback on a valid material index
					OutPart.MaterialRemap[i] = 0;
				}
			}
		}

		NaniteAssemblyData.Nodes = InAssemblyData.Nodes;

		return true;
	}

	bool BuildNanite(FSkeletalMeshRenderData& OutRenderData)
	{
		check(IsNaniteBuildEnabled());
		check(!IsNaniteAssemblyBuild() || NaniteAssemblyData.IsValid());
	
		Nanite::IBuilderModule::FInputMeshData InputMeshData;
		InitNaniteBuildInput(InputMeshData);

		const bool bGenerateFallback = AllowFallbackGeneration();
		Nanite::IBuilderModule::FOutputMeshData FallbackMeshData;

		Nanite::FResources& NaniteResources = *OutRenderData.NaniteResourcesPtr.Get();
		if (!NaniteBuilder->Build(
			NaniteResources,
			InputMeshData,
			bGenerateFallback ? &FallbackMeshData : nullptr,
			nullptr, // OutRayTracingFallbackMeshData
			nullptr, // RayTracingFallbackBuildSettings
			NaniteSettings,
			IsNaniteAssemblyBuild() ? &NaniteAssemblyData : nullptr))
		{
			return false;
		}

		// Fill out the mesh description for non-Nanite build/reduction
		if (bGenerateFallback)
		{
			check(FallbackMeshDescription);
			BuildNaniteFallbackMeshDescription(SkeletalMesh, FallbackMeshData, *FallbackMeshDescription, UnifiedBoneIndices);

			NaniteFallbackMeshSections = MoveTemp(FallbackMeshData.Sections);
		}

		return true;
	}

	Nanite::FAssemblyPartResourceRef BuildNaniteAssemblyPart()
	{
		check(IsNaniteAssemblyPartBuild());
		check(NaniteAssembliesSupported());

		Nanite::IBuilderModule::FInputMeshData InputMeshData;
		InitNaniteBuildInput(InputMeshData);

		return NaniteBuilder->BuildAssemblyPart(InputMeshData, NaniteSettings);
	}

	void RestoreClothingFromBackup()
	{
		FLODUtilities::RestoreClothingFromBackup(SkeletalMesh, ClothingBindings, LODIndex);

		FSkeletalMeshLODModel& LODModel = GetLODModel();
		LODModel.SyncronizeUserSectionsDataArray();
		LODModel.NumTexCoords = GetNumTexCoords();
		LODModel.BuildStringID = BackupBuildStringID;
	}

	void BuildMorphTargets()
	{
		if (ImportData.MorphTargetNames.Num() > 0)
		{
			FLODUtilities::BuildMorphTargets(SkeletalMesh, *SourceMeshDescription, ImportData, LODIndex, !Options.bComputeNormals, !Options.bComputeTangents, Options.bUseMikkTSpace, Options.OverlappingThresholds);
		}
	}

	void UpdateAlternateSkinWeights()
	{
		// Clear out any existing alternate skin weights from the working LOD model. We will be fully rebuilding them below.
		GetLODModel().SkinWeightProfiles.Reset();
		
		for (const FSkinWeightProfileInfo& ProfileInfo : SkeletalMesh->GetSkinWeightProfiles())
		{			
			FLODUtilities::UpdateAlternateSkinWeights(SkeletalMesh, ProfileInfo.Name, LODIndex, Options);
		}
	}

	void UpdateLODInfoVertexAttributes()
	{
		FLODUtilities::UpdateLODInfoVertexAttributes(SkeletalMesh, LODIndex, LODIndex, /*CopyAttributeValues*/true);
	}

	void PerformReductions()
	{
		FSkeletalMeshUpdateContext UpdateContext;
		UpdateContext.SkeletalMesh = SkeletalMesh;

		// We are reduce ourself in this case we reduce ourself from the original data and return true.
		if (SkeletalMesh->IsReductionActive(LODIndex))
		{
			// Update the original reduction data since we just build a new LODModel.
			if (LODInfo->ReductionSettings.BaseLOD == LODIndex && SkeletalMesh->HasMeshDescription(LODIndex))
			{
#if WITH_EDITORONLY_DATA
				if (LODIndex == 0)
				{
					if (const UAssetImportData* AssetImportData = SkeletalMesh->GetAssetImportData())
					{
						SkeletalMesh->GetLODInfo(LODIndex)->SourceImportFilename = AssetImportData->GetFirstFilename();
					}
				}
#endif
			}
			FLODUtilities::SimplifySkeletalMeshLOD(UpdateContext, LODIndex, TargetPlatform, false);
		}
		else
		{
			if (LODInfo->BonesToRemove.Num() > 0 && SkeletalMesh->GetSkeleton())
			{
				TArray<FName> BonesToRemove;
				BonesToRemove.Reserve(LODInfo->BonesToRemove.Num());
				for (const FBoneReference& BoneReference : LODInfo->BonesToRemove)
				{
					BonesToRemove.Add(BoneReference.BoneName);
				}
				MeshUtilities->RemoveBonesFromMesh(SkeletalMesh, LODIndex, &BonesToRemove);
			}
		}
	}

	void RegenerateDependentLODs()
	{
		if (bRegenDepLODs)
		{
			FLODUtilities::RegenerateDependentLODs(SkeletalMesh, LODIndex, TargetPlatform);
		}
	}

	inline bool AllowFallbackGeneration() const
	{
		return IsNaniteBuildEnabled() && bBuildNaniteFallback && FallbackMeshDescription != nullptr;
	}

private:
	void Init(const FContext* ParentContext = nullptr)
	{
		check(SkeletalMesh->GetImportedModel());
		check(SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex));

		LODInfo = SkeletalMesh->GetLODInfo(LODIndex);
		check(LODInfo);

		check(SourceMeshDescription);

		if (IsNaniteAssemblyPartBuild())
		{
			// HACK: For assembly part builds, we work on a copy
			// TODO: Remove this when we no longer have a dependency on it for Nanite build input
			LODModelClone = MakeUnique<FSkeletalMeshLODModel>();
			FSkeletalMeshLODModel::CopyStructure(LODModelClone.Get(), &SkeletalMesh->GetImportedModel()->LODModels[LODIndex]);
		}
		ImportData = FSkeletalMeshImportData::CreateFromMeshDescription(*SourceMeshDescription);
		
		// Build the skeletal mesh using mesh utilities module
		Options.FillOptions(LODInfo->BuildSettings);
		Options.TargetPlatform = TargetPlatform;
		
		// Force the normals or tangent in case the data is missing
		Options.bComputeNormals |= !ImportData.bHasNormals;
		Options.bComputeTangents |= !ImportData.bHasTangents;

		if (IsNaniteBuildEnabled())
		{
			NaniteSettings = SkeletalMesh->NaniteSettings;
			Nanite::CorrectFallbackSettings(NaniteSettings, SourceMeshDescription->Triangles().Num(), IsNaniteAssemblyBuild(), /* bIsRayTracing */ false);
			if (IsNaniteAssemblyPartBuild())
			{
				check(ParentContext != nullptr);
				Nanite::InheritAssemblySettings(NaniteSettings, ParentContext->NaniteSettings);
			}

			// Disable voxelization and warn about it if voxels are not supported
			if (NaniteSettings.ShapePreservation == ENaniteShapePreservation::Voxelize && !NaniteVoxelsSupported())
			{
				UE_LOGF(LogSkeletalMesh, Warning,
					"Skeletal mesh %ls has Shape Preservation set to \"Voxelize\", but voxels aren't enabled for the project. " "Voxelization will be disabled for the mesh. Enable Nanite Foliage in the project settings or configure " "r.Nanite.AllowVoxels=1 to enable support.",
					*SkeletalMesh->GetFullName());
				NaniteSettings.ShapePreservation = ENaniteShapePreservation::None;
			}

			if (Options.bComputeNormals && ImportData.bHasNormals)
			{
				// Import data has normals, so we always disallow recomputation
				// TODO: Desired behavior?
				Options.bComputeNormals = false;
			}

			// Never recompute tangents
			Options.bComputeTangents = false;

			// Do not cache optimize the index buffer
			Options.bCacheOptimize = false;
		}

		const int32 NumVertexInstances = SourceMeshDescription->VertexInstances().GetArraySize();
		if (NumVertexInstances >= 100000 * 3)
		{
			// Just like static mesh, we disable cache optimization on very high poly 
			// meshes because they are likely not for game rendering, or they are intended
			// for rendering with Nanite.
			Options.bCacheOptimize = false;
		}

		if (IsNaniteAssemblyBuild())
		{
			// Compute the composed ref pose matrices if they are in need of updating
			SkeletalMesh->CalculateInvRefMatrices();

			// The assembly data needs the skeleton's composed ref pose
			const uint32 NumBones = SkeletalMesh->GetRefSkeleton().GetRawBoneNum();
			NaniteAssemblyData.ComposedRefPose.SetNumUninitialized(NumBones);
			for (uint32 i = 0; i < NumBones; ++i)
			{
				NaniteAssemblyData.ComposedRefPose[i] = FMatrix44f(SkeletalMesh->GetComposedRefPoseMatrix(i));
			}

			// When the skeletal mesh uses unified bone indices, give the Nanite builder a remap
			// table so it can convert assembly bone influences from skeleton space to unified
			// space. Without this, vertex bone indices produced from assembly data end up in a
			// different index space than vertex bone indices encoded by InitNaniteBuildInput,
			// and the fallback decode path mis-interprets them.
			if (UnifiedBoneIndices.IsValid())
			{
				NaniteAssemblyData.SkeletonToUnifiedBoneIndex.SetNumUninitialized(NumBones);
				for (uint32 i = 0; i < NumBones; ++i)
				{
					NaniteAssemblyData.SkeletonToUnifiedBoneIndex[i] = UnifiedBoneIndices.GetSkeletonToUnifiedIndex(i);
				}
			}
		}
	}

	void InitNaniteBuildInput(Nanite::IBuilderModule::FInputMeshData& InputMeshData)
	{
		check(IsNaniteBuildEnabled());

		const FSkeletalMeshLODModel& LODModel = GetLODModel();

		// Build new vertex buffers
		InputMeshData.NumTexCoords = LODModel.NumTexCoords;

		InputMeshData.MaterialIndices.SetNumUninitialized(LODModel.IndexBuffer.Num() / 3);

		InputMeshData.Vertices.Position.SetNumUninitialized(LODModel.NumVertices);
		InputMeshData.Vertices.TangentX.SetNumUninitialized(LODModel.NumVertices);
		InputMeshData.Vertices.TangentY.SetNumUninitialized(LODModel.NumVertices);
		InputMeshData.Vertices.TangentZ.SetNumUninitialized(LODModel.NumVertices);

		InputMeshData.Vertices.UVs.SetNum(LODModel.NumTexCoords);
		for (uint32 UVCoord = 0; UVCoord < LODModel.NumTexCoords; ++UVCoord)
		{
			InputMeshData.Vertices.UVs[UVCoord].SetNumUninitialized(LODModel.NumVertices);
		}

		// We can save memory by figuring out the max number of influences across all sections instead of allocating MAX_TOTAL_INFLUENCES
		// Also check if any of the sections actually require 16bit, or if 8bit will suffice
		bool b16BitSkinning = false;
		InputMeshData.NumBoneInfluences = 0;
		for (const FSkelMeshSection& Section : LODModel.Sections)
		{
			InputMeshData.NumBoneInfluences = FMath::Max< uint8 >(InputMeshData.NumBoneInfluences, uint32(Section.MaxBoneInfluences));
			b16BitSkinning |= Section.Use16BitBoneIndex();
		}

		InputMeshData.Vertices.BoneIndices.SetNum(InputMeshData.NumBoneInfluences);
		InputMeshData.Vertices.BoneWeights.SetNum(InputMeshData.NumBoneInfluences);
		for (uint32 Influence = 0; Influence < InputMeshData.NumBoneInfluences; ++Influence)
		{
			InputMeshData.Vertices.BoneIndices[Influence].SetNumZeroed(LODModel.NumVertices);
			InputMeshData.Vertices.BoneWeights[Influence].SetNumZeroed(LODModel.NumVertices);
		}

		// TODO: Nanite-Skinning
		//InputMeshData.Vertices.Color.SetNumUninitialized(LODModel.NumVertices);

		InputMeshData.TriangleIndices = LODModel.IndexBuffer;

		uint32 CheckIndices = 0;
		uint32 CheckVertices = 0;

		bool bLogOnce = false;

		for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); ++SectionIndex)
		{
			const FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

			check(CheckIndices  == Section.BaseIndex);
			check(CheckVertices == Section.BaseVertexIndex);

			for (int32 VertIndex = 0; VertIndex < Section.SoftVertices.Num(); ++VertIndex)
			{
				const FSoftSkinVertex& SoftVertex = Section.SoftVertices[VertIndex];

				InputMeshData.Vertices.Position[Section.BaseVertexIndex + VertIndex] = SoftVertex.Position;
				InputMeshData.Vertices.TangentX[Section.BaseVertexIndex + VertIndex] = SoftVertex.TangentX;
				InputMeshData.Vertices.TangentY[Section.BaseVertexIndex + VertIndex] = SoftVertex.TangentY;
				InputMeshData.Vertices.TangentZ[Section.BaseVertexIndex + VertIndex] = SoftVertex.TangentZ;

				InputMeshData.VertexBounds += SoftVertex.Position;

				for (uint32 UVCoord = 0; UVCoord < LODModel.NumTexCoords; ++UVCoord)
				{
					InputMeshData.Vertices.UVs[UVCoord][Section.BaseVertexIndex + VertIndex] = SoftVertex.UVs[UVCoord];
				}

				for (int32 Influence = 0; Influence < Section.MaxBoneInfluences; ++Influence)
				{
					FBoneIndexType BoneIndex = Section.BoneMap[SoftVertex.InfluenceBones[Influence]];

					if (UnifiedBoneIndices.IsValid())
					{
						const FBoneIndexType PreRemapBoneIndex = BoneIndex;
						const FBoneIndexType PreRemapLocalIndex = SoftVertex.InfluenceBones[Influence];
						const uint16 InfluenceWeight = SoftVertex.InfluenceWeights[Influence];

						// Remap the skeleton index into unified bone index space.
						BoneIndex = UnifiedBoneIndices.GetSkeletonToUnifiedIndex(PreRemapBoneIndex);

						if (BoneIndex == INVALID_BONE_INDEX && InfluenceWeight != 0)
						{
							if (!bLogOnce)
							{
								UE_LOG(LogSkeletalMesh, Error,
									TEXT("Nanite ISKM encode: skeleton bone %u (from Section.BoneMap[%u]) has no unified mapping "
										 "but vertex has non-zero weight %u. Nanite vertex would reference an invalid transform slot. "
										 "Falling back to bone 0 (root) with zero weight for this influence."),
									(uint32)PreRemapBoneIndex,
									(uint32)PreRemapLocalIndex,
									(uint32)InfluenceWeight);
								bLogOnce = true;
							}
							BoneIndex = 0;
							InputMeshData.Vertices.BoneIndices[Influence][Section.BaseVertexIndex + VertIndex] = 0;
							InputMeshData.Vertices.BoneWeights[Influence][Section.BaseVertexIndex + VertIndex] = 0;
							continue;
						}
					}

					InputMeshData.Vertices.BoneIndices[Influence][Section.BaseVertexIndex + VertIndex] = BoneIndex;
					InputMeshData.Vertices.BoneWeights[Influence][Section.BaseVertexIndex + VertIndex] = SoftVertex.InfluenceWeights[Influence];
				}

				//InputMeshData.Vertices.Color[Section.BaseVertexIndex + VertIndex] = SoftVertex.Color;
			}

			for (uint32 MaterialIndex = 0; MaterialIndex < Section.NumTriangles; ++MaterialIndex)
			{
				InputMeshData.MaterialIndices[(CheckIndices / 3) + MaterialIndex] = Section.MaterialIndex;
			}

			CheckIndices += Section.NumTriangles * 3;
			CheckVertices += Section.NumVertices;
		}

		check(CheckVertices == LODModel.NumVertices);
		check(CheckIndices == LODModel.IndexBuffer.Num());

		InputMeshData.TriangleCounts.Add(LODModel.IndexBuffer.Num() / 3);

		InputMeshData.Sections = Nanite::BuildMeshSections(LODModel.Sections);
	}

	FString BackupBuildStringID;
	TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ClothingBindings;
};

} // namespace SkeletalMeshBuilderPrivate


FSkeletalMeshBuilder::FSkeletalMeshBuilder()
{
}

static bool FinalizeContext(FScopedSlowTask& SlowTask, SkeletalMeshBuilderPrivate::FContext& Context)
{
	// Re-apply the morph target
	SlowTask.EnterProgressFrame(1.0f, NSLOCTEXT("SkeltalMeshBuilder", "RebuildMorphTarget", "Rebuilding morph targets..."));
	Context.BuildMorphTargets();

	// Re-apply the alternate skinning it must be after the inline reduction
	SlowTask.EnterProgressFrame(1.0f, NSLOCTEXT("SkeltalMeshBuilder", "RebuildAlternateSkinning", "Rebuilding alternate skinning..."));
	Context.UpdateAlternateSkinWeights();

	// Copy vertex attribute definitions and their values from the import model.
	Context.UpdateLODInfoVertexAttributes();

	SlowTask.EnterProgressFrame(1.0f, NSLOCTEXT("SkeltalMeshBuilder", "RegenerateLOD", "Regenerate LOD..."));
	Context.PerformReductions();

	// Re-apply the clothing using the UserSectionsData, this will ensure we remap correctly the cloth if the reduction has change the number of sections
	SlowTask.EnterProgressFrame(1.0f, NSLOCTEXT("SkeltalMeshBuilder", "RebuildClothing", "Rebuilding clothing..."));
	Context.RestoreClothingFromBackup();

	SlowTask.EnterProgressFrame(1.0f, NSLOCTEXT("SkeltalMeshBuilder", "RegenerateDependentLODs", "Regenerate Dependent LODs..."));
	Context.RegenerateDependentLODs();

	return true;
}

static bool BuildNanite(FScopedSlowTask& SlowTask, SkeletalMeshBuilderPrivate::FContext& Context, FSkeletalMeshRenderData& OutRenderData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshBuilder::BuildNanite);

	check(Context.IsNaniteBuildEnabled());

	ClearNaniteResources(OutRenderData.NaniteResourcesPtr);

	if (Context.IsNaniteAssemblyBuild())
	{
		SlowTask.EnterProgressFrame(1.0f, NSLOCTEXT("SkeltalMeshBuilder", "BuildingNaniteAssembly", "Building Nanite assembly parts..."));
		if (!Context.BuildNaniteAssemblyData())
		{
			return false;
		}
	}

	SlowTask.EnterProgressFrame(1.0f, NSLOCTEXT("SkeltalMeshBuilder", "BuildingNaniteData", "Building Nanite data..."));

	bool bBuildSuccess = Context.BuildNanite(OutRenderData);
	if (!bBuildSuccess)
	{
		UE_LOGF(LogSkeletalMesh, Error, "Failed to build Nanite for skeletal mesh. See previous line(s) for details.");
	}

	return bBuildSuccess;
}

bool FSkeletalMeshBuilder::Build(FSkeletalMeshRenderData& OutRenderData, const FSkeletalMeshBuildParameters& SkeletalMeshBuildParameters)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshBuilder::Build);

	FMeshDescription* MeshDescriptionPtr = SkeletalMeshBuildParameters.SkeletalMesh->GetMeshDescription(SkeletalMeshBuildParameters.LODIndex);
	if (!MeshDescriptionPtr)
	{
		UE_LOGF(LogSkeletalMesh, Error, "Failed to build skeletal mesh. Invalid Mesh Description at LODIndex %d", SkeletalMeshBuildParameters.LODIndex);
		return false;
	}
	FSkeletalMeshConstAttributes MeshAttributes(*MeshDescriptionPtr);

	const bool bHasClothing = SkeletalMeshBuildParameters.SkeletalMesh->GetMeshClothingAssets().Num() > 0;
	const bool bHasMorphTargets = MeshAttributes.GetMorphTargetNames().Num() > 0;
	const bool bBuildNanite = SkeletalMeshBuildParameters.SkeletalMesh->IsNaniteEnabled() && SkeletalMeshBuildParameters.LODIndex == 0;
	const bool bAssemblyBuild = bBuildNanite && SkeletalMeshBuildParameters.SkeletalMesh->IsNaniteAssembly();

	// TODO: Some issues to work out with missing triangles, and corrupt TSB if recompute normals/tangents is enabled.
	const bool bBuildNaniteFallback = bAssemblyBuild;// bBuildNanite && (bAssemblyBuild || (!bHasClothing && !bHasMorphTargets));

	const float TaskTotal = 
		5.01f /*FinalizeContext*/ + 
		1.0f /* BuildLODModel */ +
		(bBuildNanite ? 1.0f : 0.0f) +
		(bAssemblyBuild ? 1.0f : 0.0f);

	FScopedSlowTask SlowTask(TaskTotal, NSLOCTEXT("SkeletalMeshBuilder", "BuildingSkeletalMeshLOD", "Building skeletal mesh LOD"));
	SlowTask.MakeDialog();

	// Prevent any PostEdit change during the build
	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMeshBuildParameters.SkeletalMesh, false, false);

	if (bBuildNaniteFallback)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSkeletalMeshBuilder::BuildNanite);

		FMeshDescription FallbackMeshDescription;
		Nanite::FMeshDataSectionArray NaniteFallbackMeshSections;
		{
			SkeletalMeshBuilderPrivate::FContext NaniteContext(
				SkeletalMeshBuildParameters,
				MeshDescriptionPtr,
				&FallbackMeshDescription,
				bBuildNanite,
				bBuildNaniteFallback,
				{},
				OutRenderData.GetUnifiedBoneMap()
			);
			check(NaniteContext.IsNaniteBuildEnabled());

			// Unbind any existing clothing assets before we reimport the geometry
			NaniteContext.UnbindClothingAndBackup();

			NaniteContext.BuildLODModel();

			if (!BuildNanite(SlowTask, NaniteContext, OutRenderData))
			{
				return false;
			}

			NaniteFallbackMeshSections = MoveTemp(NaniteContext.NaniteFallbackMeshSections);
		}

		SkeletalMeshBuilderPrivate::FContext Context(
			SkeletalMeshBuildParameters,
			&FallbackMeshDescription,
			nullptr /* FallbackMeshDescription */,
			false /* bBuildNanite */,
			false /* bBuildNaniteFallback */,
			MoveTemp(NaniteFallbackMeshSections),
			OutRenderData.GetUnifiedBoneMap()
		);

		SlowTask.EnterProgressFrame(1.0f);
		Context.BuildLODModel();

		return FinalizeContext(SlowTask, Context);
	}
	else
	{
		SkeletalMeshBuilderPrivate::FContext Context(
			SkeletalMeshBuildParameters,
			MeshDescriptionPtr,
			nullptr /* FallbackMeshDescription */,
			bBuildNanite,
			false /* bBuildNaniteFallback */,
			{},
			OutRenderData.GetUnifiedBoneMap()
		);

		// Unbind any existing clothing assets before we reimport the geometry
		Context.UnbindClothingAndBackup();

		SlowTask.EnterProgressFrame(1.0f);
		Context.BuildLODModel();

		if (bBuildNanite)
		{
			if (!BuildNanite(SlowTask, Context, OutRenderData))
			{
				return false;
			}
		}

		return FinalizeContext(SlowTask, Context);
	}
}
