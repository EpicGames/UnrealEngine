// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshBuilder.h"

#include "BuildOptimizationHelper.h"
#include "Components.h"
#include "Engine/StaticMesh.h"
#include "IMeshReductionInterfaces.h"
#include "IMeshReductionManagerModule.h"
#include "Logging/StructuredLog.h"
#include "MeshBuild.h"
#include "MeshDescriptionHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "PhysicsEngine/BodySetup.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "StaticMeshResources.h"
#include "Math/Bounds.h"
#include "NaniteBuilder.h"
#include "NaniteHelper.h"
#include "Rendering/NaniteResources.h"
#include "Interfaces/ITargetPlatform.h"
#include "RenderMath.h"

DEFINE_LOG_CATEGORY(LogStaticMeshBuilder);

void BuildAllBufferOptimizations(
	struct FStaticMeshLODResources& StaticMeshLOD,
	const struct FMeshBuildSettings& LODBuildSettings,
	TArray< uint32 >& IndexBuffer,
	bool bNeeds32BitIndices,
	const FConstMeshBuildVertexView& BuildVertices
);

FStaticMeshBuilder::FStaticMeshBuilder()
{

}

static bool UseNativeQuadraticReduction()
{
	// Are we using our tool, or simplygon?  The tool is only changed during editor restarts
	IMeshReduction* ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface").GetStaticMeshReductionInterface();

	FString VersionString = ReductionModule->GetVersionString();
	TArray<FString> SplitVersionString;
	VersionString.ParseIntoArray(SplitVersionString, TEXT("_"), true);

	bool bUseQuadricSimplier = SplitVersionString[0].Equals("QuadricMeshReduction");
	return bUseQuadricSimplier;
}


/**
 * Compute bounding box and sphere from position buffer
 */
static void ComputeBoundsFromPositionBuffer(const FPositionVertexBuffer& UsePositionBuffer, FBoxSphereBounds& BoundsOut)
{
	// Calculate the bounding box.
	FBounds3f Bounds;
	for (uint32 VertexIndex = 0; VertexIndex < UsePositionBuffer.GetNumVertices(); VertexIndex++)
	{
		Bounds += UsePositionBuffer.VertexPosition(VertexIndex);
	}
	
	// Calculate the bounding sphere, using the center of the bounding box as the origin.
	FVector3f Center = Bounds.GetCenter();
	float RadiusSqr = 0.0f;
	for (uint32 VertexIndex = 0; VertexIndex < UsePositionBuffer.GetNumVertices(); VertexIndex++)
	{
		RadiusSqr = FMath::Max(	RadiusSqr, ( UsePositionBuffer.VertexPosition(VertexIndex) - Center ).SizeSquared() );
	}

	BoundsOut.Origin = FVector(Center);
	BoundsOut.BoxExtent = FVector(Bounds.GetExtent());
	BoundsOut.SphereRadius = FMath::Sqrt( RadiusSqr );
}


/**
 * Compute bounding box and sphere from vertices
 */
static void ComputeBoundsFromVertexList(const TArray<FStaticMeshBuildVertex>& Vertices, FBoxSphereBounds& BoundsOut)
{
	// Calculate the bounding box.
	FBounds3f Bounds;
	for (int32 VertexIndex = 0; VertexIndex < Vertices.Num(); VertexIndex++)
	{
		Bounds += Vertices[VertexIndex].Position;
	}
	
	// Calculate the bounding sphere, using the center of the bounding box as the origin.
	FVector3f Center = Bounds.GetCenter();
	float RadiusSqr = 0.0f;
	for (int32 VertexIndex = 0; VertexIndex < Vertices.Num(); VertexIndex++)
	{
		RadiusSqr = FMath::Max(	RadiusSqr, ( Vertices[VertexIndex].Position - Center ).SizeSquared() );
	}

	BoundsOut.Origin = FVector(Center);
	BoundsOut.BoxExtent = FVector(Bounds.GetExtent());
	BoundsOut.SphereRadius = FMath::Sqrt( RadiusSqr );
}

static void ScaleStaticMeshVertex(
	FVector3f& Position,
	FVector3f& TangentX,
	FVector3f& TangentY,
	FVector3f& TangentZ,
	FVector3f Scale,
	bool bNeedTangents,
	bool bUseLegacyTangentScaling
)
{
	Position *= Scale;
	if (bNeedTangents)
	{
		if (bUseLegacyTangentScaling)
		{
			// Apply incorrect inverse scale to tangents to match an old bug, for legacy assets only
			TangentX /= Scale;
			TangentY /= Scale;
		}
		else
		{
			// Tangents should transform by directly applying the same scale as the geometry; it's only the normal that needs an inverse scale
			TangentX *= Scale;
			TangentY *= Scale;
		}
		TangentX.Normalize();
		TangentY.Normalize();
	}
	else
	{
		TangentX = FVector3f(1.0f, 0.0f, 0.0f);
		TangentY = FVector3f(0.0f, 1.0f, 0.0f);
	}
	TangentZ /= Scale;
	TangentZ.Normalize();
}

struct FStaticMeshNaniteBuildContext
{
	FMeshNaniteSettings Settings;
	UStaticMesh* StaticMesh						= nullptr;
	const ITargetPlatform* TargetPlatform		= nullptr;
	const FStaticMeshSourceModel* SourceModel	= nullptr;
	Nanite::IBuilderModule* Builder 			= nullptr;

	bool bIsAssembly 			: 1	= false;
	bool bIsAssemblyPart		: 1	= false;
	bool bHiResSourceModel 		: 1	= false;

	bool IsValid() const { return StaticMesh != nullptr; }
};

static bool PrepareNaniteStaticMeshBuild(
	FStaticMeshNaniteBuildContext& OutContext,
	UStaticMesh* StaticMesh,
	const ITargetPlatform* TargetPlatform,
	FStaticMeshNaniteBuildContext* ParentContext = nullptr)
{
	check(ParentContext == nullptr || ParentContext->IsValid());

	if (ParentContext == nullptr && !StaticMesh->IsNaniteEnabled())
	{
		// We don't need to build Nanite for this static mesh
		return false;
	}

	const bool bIsAssembly = StaticMesh->GetNaniteSettings().NaniteAssemblyData.IsValid();
	const bool bTargetSupportsNanite = DoesTargetPlatformSupportNanite(TargetPlatform);
	FStaticMeshSourceModel& LOD0SourceModel = StaticMesh->GetSourceModel(0);
	FStaticMeshSourceModel& HiResSourceModel = StaticMesh->GetHiResSourceModel();
	const bool bHasHiResSourceModel = HiResSourceModel.IsMeshDescriptionValid();

	if (!bTargetSupportsNanite && bHasHiResSourceModel)
	{
		// If the target we're building for doesn't support Nanite and we have a hi-res source model, then we don't need to build
		// Nanite, since LOD0 will remain unsimplified.
		// NOTE: This is an optimization for non-Nanite build times and is only valid because we know the DDC key for static mesh
		// cache will be different for meshes with hi-res source data between Nanite and non-Nanite platforms. Otherwise, this
		// would have the potential to cause non-Nanite platforms to cache static mesh data without Nanite resources that Nanite
		// platforms would subsequently load from the DDC.
		return false;
	}

	const FMeshDescription* LOD0MeshDescription = LOD0SourceModel.GetOrCacheMeshDescription();
	if (LOD0MeshDescription == nullptr)
	{
		UE_LOGF(LogStaticMeshBuilder, Error, "Invalid mesh description during Nanite build [%ls].", *StaticMesh->GetFullName());
		return false;
	}
	if (LOD0MeshDescription->IsEmpty())
	{
		UE_LOGF(LogStaticMeshBuilder, Error, "Empty mesh description during Nanite build [%ls].", *StaticMesh->GetFullName());
		return false;
	}

	// Only do Nanite build for the hi-res source model if we have one, the target platform supports Nanite, AND the mesh description
	// is well-formed. In all other cases, we will build Nanite from LOD0. This will replace the output VertexBuffers/etc with
	// the fractional Nanite cut to be stored as LOD0 RenderData.
	// NOTE: We also want to use LOD0 for targets that do not support Nanite (even if a hi-res source model was provided)
	// so that it generates the fallback, in which case the Nanite bulk will be stripped
	bool bUseHiResSourceModel = false;
	if (bTargetSupportsNanite && bHasHiResSourceModel)
	{
		if (const FMeshDescription* HiResMeshDescription = HiResSourceModel.GetOrCacheMeshDescription())
		{
			if (HiResMeshDescription->IsEmpty())
			{
				UE_LOGF(LogStaticMeshBuilder, Display,
					"Invalid hi-res mesh description during Nanite build [%ls]. The hi-res mesh is empty. " "This is not supported and LOD 0 will be used as a fallback to build Nanite data.",
					*StaticMesh->GetFullName());
			}
			else
			{
				// Make sure hi-res mesh data has the same amount of sections. If not, rendering bugs and issues will show
				// up because the Nanite render must use the LOD 0 sections.
				const int32 NumLOD0PolyGroups = LOD0MeshDescription->PolygonGroups().Num();
				if (HiResMeshDescription->PolygonGroups().Num() > NumLOD0PolyGroups)
				{
					UE_LOGF(LogStaticMeshBuilder, Display,
						"Invalid hi-res mesh description during Nanite build [%ls]. " "The number of sections from the hires mesh is higher than LOD 0 section count. " "This is not supported and LOD 0 will be used as a fallback to build Nanite data.",
						*StaticMesh->GetFullName());
				}
				else
				{
					if (HiResMeshDescription->PolygonGroups().Num() < NumLOD0PolyGroups)
					{
						UE_LOGF(LogStaticMeshBuilder, Display,
							"Nanite hi-res mesh description for [%ls] has fewer sections than lod 0. " "Verify you have the proper material id result when Nanite is turned on.",
							*StaticMesh->GetFullName());
					}
					bUseHiResSourceModel = true;
				}
			}
		}
	}
	
	OutContext.Settings = StaticMesh->GetNaniteSettings();
	Nanite::CorrectFallbackSettings(OutContext.Settings, LOD0MeshDescription->Triangles().Num(), bIsAssembly, /* bIsRayTracing */ false);

	const bool bIsAssemblyPart = ParentContext != nullptr;
	if (bIsAssemblyPart)
	{
		Nanite::InheritAssemblySettings(OutContext.Settings, ParentContext->Settings);
	}

	// Disable voxelization and warn about it if voxels are not supported
	if (OutContext.Settings.ShapePreservation == ENaniteShapePreservation::Voxelize && !NaniteVoxelsSupported())
	{
		UE_LOGF(LogStaticMeshBuilder, Warning,
			"Static mesh %ls has Shape Preservation set to \"Voxelize\", but voxels aren't enabled for the project. " "Voxelization will be disabled for the mesh. Enable Nanite Foliage in the project settings or configure " "r.Nanite.AllowVoxels=1 to enable support.",
			*StaticMesh->GetFullName());
		OutContext.Settings.ShapePreservation = ENaniteShapePreservation::None;
	}

	OutContext.StaticMesh 				= StaticMesh;
	OutContext.SourceModel				= bUseHiResSourceModel ? &HiResSourceModel : &LOD0SourceModel;
	OutContext.TargetPlatform			= TargetPlatform;
	OutContext.Builder					= bIsAssemblyPart ? ParentContext->Builder : &Nanite::IBuilderModule::Get();
	OutContext.bIsAssembly 				= bIsAssembly;
	OutContext.bIsAssemblyPart 			= bIsAssemblyPart;
	OutContext.bHiResSourceModel		= bUseHiResSourceModel;

	return true;
}

static bool InitNaniteBuildInput(
	FStaticMeshNaniteBuildContext& Context,
	Nanite::IBuilderModule::FInputMeshData& OutData,
	FBoxSphereBounds& OutVertexBounds)
{
	FMeshDescription MeshDescription; 
	if (!Context.SourceModel->CloneMeshDescription(MeshDescription))
	{
		UE_LOGF(LogStaticMeshBuilder, Error,
			"Failed to clone mesh description during Nanite build [%ls].",
			*Context.StaticMesh->GetFullName());
		return false;
	}

	if (MeshDescription.IsEmpty())
	{
		UE_LOGF(LogStaticMeshBuilder, Error,
			"Cannot build an empty mesh description during Nanite build [%ls].",
			*Context.StaticMesh->GetFullName());
		return false;
	}

	FMeshBuildSettings& BuildSettings = Context.StaticMesh->GetSourceModel(0).BuildSettings;

	// Only build tangents if they are explicitly enabled or we're going to be injecting this vertex data directly into
	// LOD0 of a generated fallback
	const bool bFallbackUsesInputMeshData =
		!Context.bIsAssemblyPart &&
		!Context.bHiResSourceModel &&
		Context.Settings.FallbackPercentTriangles == 1.0f &&
		Context.Settings.FallbackRelativeError == 0.0f;
	const bool bNeedTangents = Context.Settings.bExplicitTangents || bFallbackUsesInputMeshData;

	// compute tangents, lightmap UVs, etc
	FMeshDescriptionHelper MeshDescriptionHelper(&BuildSettings);
	MeshDescriptionHelper.SetupRenderMeshDescription(Context.StaticMesh, MeshDescription, true, bNeedTangents);

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	// Prepare the PerSectionIndices array so we can optimize the index buffer for the GPU
	TArray<TArray<uint32>> PerSectionIndices;
	PerSectionIndices.AddDefaulted(MeshDescription.PolygonGroups().Num());

	FStaticMeshSectionArray StaticMeshSections;
	StaticMeshSections.Empty(MeshDescription.PolygonGroups().Num());

	// We only need this to de-duplicate vertices inside of BuildVertexBuffer
	// (And only if there are overlapping corners in the mesh description).
	TArray<int32> RemapVerts;

	// Nanite does not need the wedge map returned (mainly used by non-Nanite mesh painting).
	const bool bNeedWedgeMap = false;
	TArray<int32> WedgeMap;

	// Build the vertex and index buffer
	UE::Private::StaticMeshBuilder::BuildVertexBuffer(
		Context.StaticMesh,
		MeshDescription,
		BuildSettings,
		WedgeMap,
		StaticMeshSections,
		PerSectionIndices,
		OutData.Vertices,
		MeshDescriptionHelper.GetOverlappingCorners(),
		RemapVerts,
		OutVertexBounds,
		bNeedTangents,
		bNeedWedgeMap);

	// Concatenate the per-section index buffers.
	bool bNeeds32BitIndices = false;
	UE::Private::StaticMeshBuilder::BuildCombinedSectionIndices(
		PerSectionIndices,
		StaticMeshSections,
		OutData.TriangleIndices,
		bNeeds32BitIndices);

	OutData.Sections = Nanite::BuildMeshSections(StaticMeshSections);

	// Nanite build requires the section material indices to have already been resolved from the SectionInfoMap
	// as the indices are baked into the FMaterialTriangles.
	for (int32 SectionIndex = 0; SectionIndex < OutData.Sections.Num(); SectionIndex++)
	{
		OutData.Sections[SectionIndex].MaterialIndex = Context.StaticMesh->GetSectionInfoMap().Get(0, SectionIndex).MaterialIndex;
	}

	OutData.VertexBounds.Min = FVector3f(OutVertexBounds.Origin - OutVertexBounds.BoxExtent);
	OutData.VertexBounds.Max = FVector3f(OutVertexBounds.Origin + OutVertexBounds.BoxExtent);

	TVertexInstanceAttributesRef<FVector2f const> VertexInstanceUVs = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
	OutData.NumTexCoords = VertexInstanceUVs.IsValid() ? VertexInstanceUVs.GetNumChannels() : 0;

	const uint32 TriangleCount = OutData.TriangleIndices.Num() / 3;
	OutData.TriangleCounts.Add(TriangleCount);

	if (!Context.Builder->BuildMaterialIndices(OutData.Sections, TriangleCount, OutData.MaterialIndices))
	{
		UE_LOGFMT_NSLOC(LogStaticMesh, Warning, "StaticMesh", "NaniteBuildStaticMeshError", "Failed to build Nanite from static mesh. See previous line(s) for details.");
		return false;
	}

	return true;
}

static Nanite::FAssemblyPartResourceRef BuildNaniteAssemblyPart(FStaticMeshNaniteBuildContext& ParentContext, UStaticMesh* PartMesh)
{
	FStaticMeshNaniteBuildContext ChildContext;
	if (!PrepareNaniteStaticMeshBuild(ChildContext, PartMesh, ParentContext.TargetPlatform, &ParentContext))
	{
		return nullptr;
	}

	Nanite::IBuilderModule::FInputMeshData InputMeshData;
	FBoxSphereBounds VertexBounds;
	if (!InitNaniteBuildInput(ChildContext, InputMeshData, VertexBounds))
	{
		return nullptr;
	}

	return ChildContext.Builder->BuildAssemblyPart(
		InputMeshData,
		ChildContext.Settings
	);
}

static bool InitNaniteAssemblyData(
	FStaticMeshNaniteBuildContext& Context,
	Nanite::FInputAssemblyData& OutData)
{
	check(Context.bIsAssembly);

	if (!NaniteAssembliesSupported())
	{
		UE_LOGFMT_NSLOC(LogStaticMesh, Warning, "StaticMesh", "NaniteBuildAssemblyNotEnabledError",
			"Failed to build Nanite Assembly static mesh {MeshPath}. Nanite Assemblies are not enabled for the project. "
			"Enable Nanite Foliage in the project settings or configure r.Nanite.AllowAssemblies=1 to enable support.",
			("MeshPath", Context.StaticMesh->GetPathName()));
		return false;
	}
	else if (!Context.StaticMesh->HasCachedNaniteAssemblyReferences())
	{
		UE_LOGFMT_NSLOC(LogStaticMesh, Warning, "StaticMesh", "NaniteBuildAssemblyUncachedError",
			"Failed to build Nanite Assembly static mesh {MeshPath}. The referenced static meshes were not cached before build.",
			("MeshPath", Context.StaticMesh->GetPathName()));
		return false;
	}

	// Get the assembly references that should have been resolved before build
	const FNaniteAssemblyData& AssemblyData = Context.Settings.NaniteAssemblyData;
	const TArray<TObjectPtr<UStaticMesh>>& PartReferences = Context.StaticMesh->GetCachedNaniteAssemblyReferences();
	check(PartReferences.Num() == AssemblyData.Parts.Num());

	TMap<UStaticMesh*, Nanite::FAssemblyPartResourceRef> ResourceLookup;
	ResourceLookup.Reserve(PartReferences.Num());

	for (int32 PartIndex = 0; PartIndex < AssemblyData.Parts.Num(); ++PartIndex)
	{
		Nanite::FAssemblyPartResourceRef Resource;

		if (UStaticMesh* PartMesh = PartReferences[PartIndex])
		{
			if (Nanite::FAssemblyPartResourceRef* ExistingResource = ResourceLookup.Find(PartMesh))
			{
				Resource = *ExistingResource;
			}
			else
			{
				Resource = BuildNaniteAssemblyPart(Context, PartMesh);
				ResourceLookup.Add(PartMesh, Resource);
			}
		}

		if (!Resource.IsValid())
		{
			UE_LOGFMT_NSLOC(LogStaticMesh, Warning, "StaticMesh", "NaniteBuildAssemblyPartError",
				"Failed to build Nanite assembly part from static mesh ({PartPath}). See previous line(s) for details.",
				("PartPath", AssemblyData.Parts[PartIndex].MeshObjectPath.GetAssetName()));
			return false;
		}

		auto& OutPart = OutData.Parts.AddDefaulted_GetRef();
		OutPart.Resource = Resource;
		
		// Apply the material remap
		const TArray<int32>& MaterialRemap = AssemblyData.Parts[PartIndex].MaterialRemap;
		for (int32 i = 0; i < Nanite::MaxSectionArraySize; ++i)
		{
			if (MaterialRemap.Num() == 0)
			{
				// No remaps = match indices
				OutPart.MaterialRemap[i] = i;
			}
			else if (MaterialRemap.IsValidIndex(i))
			{
				OutPart.MaterialRemap[i] = MaterialRemap[i];
			}
			else
			{
				// Index is unrepresented in the remap (may not be a valid index). Fallback on a valid material index
				OutPart.MaterialRemap[i] = 0;
			}
		}
	}

	OutData.Nodes = AssemblyData.Nodes;

	return true;
}

static void BuildNaniteFallbackMeshDescription(
	FStaticMeshNaniteBuildContext& Context,
	const Nanite::IBuilderModule::FOutputMeshData& InMeshData,
	FMeshDescription& OutMesh
)
{
	OutMesh.Empty();
	
	// Lod zero was built with scaling build settings, we have to remove the scaling from the data since the other LODs build will also apply the scaling.
	const FVector3f InverseBuildScale = FVector3f(FVector(1.0) / Context.SourceModel->BuildSettings.BuildScale3D);
	const bool bBuildScaleActive = !InverseBuildScale.Equals(FVector3f(1.0f), UE_SMALL_NUMBER);
	const bool bUseLegacyTangentScaling = Context.StaticMesh->GetLegacyTangentScaling();

	FStaticMeshAttributes Attributes(OutMesh);
	Attributes.Register();

	const int32 NumVertices = InMeshData.Vertices.Position.Num();
	const int32 NumUVChannels = InMeshData.Vertices.UVs.Num();
	const int32 NumTriangles = InMeshData.TriangleIndices.Num() / 3;
	const int32 NumPolyGroups = InMeshData.Sections.Num();

	OutMesh.ReserveNewVertices(NumVertices);
	OutMesh.ReserveNewVertexInstances(NumVertices);
	OutMesh.ReserveNewTriangles(NumTriangles);
	OutMesh.ReserveNewPolygonGroups(NumPolyGroups);

	OutMesh.SetNumUVChannels(NumUVChannels);
	OutMesh.VertexInstanceAttributes().SetAttributeChannelCount(MeshAttribute::VertexInstance::TextureCoordinate, NumUVChannels);
	for (int32 UVChannelIndex = 0; UVChannelIndex < NumUVChannels; ++UVChannelIndex)
	{
		OutMesh.ReserveNewUVs(NumVertices, UVChannelIndex);
	}

	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
	TPolygonGroupAttributesRef<FName> PolygonGroupMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();

	for (int32 InVertIndex = 0; InVertIndex < NumVertices; ++InVertIndex)
	{
		const FVertexID VertexID(InVertIndex);
		const FVertexInstanceID VertexInstanceID(InVertIndex);

		// TODO: Deduplicate vertex positions?
		OutMesh.CreateVertexWithID(VertexID);
		OutMesh.CreateVertexInstanceWithID(VertexInstanceID, VertexID);

		FVector3f Position = InMeshData.Vertices.Position[InVertIndex];
		FVector3f TangentX = InMeshData.Vertices.TangentX[InVertIndex];
		FVector3f TangentY = InMeshData.Vertices.TangentY[InVertIndex];
		FVector3f TangentZ = InMeshData.Vertices.TangentZ[InVertIndex];

		if (bBuildScaleActive)
		{
			ScaleStaticMeshVertex(
				Position,
				TangentX,
				TangentY,
				TangentZ,
				InverseBuildScale,
				true, // bNeedTangents
				bUseLegacyTangentScaling
			);
		}

		// No CurveRadius for static meshes
		check(InMeshData.Vertices.CurveRadius.Num() == 0);

		const float BinormalSign = GetBasisDeterminantSign(FVector(TangentX), FVector(TangentY), FVector(TangentZ));
		const FColor Color = InMeshData.Vertices.Color.IsValidIndex(InVertIndex) ?
			InMeshData.Vertices.Color[InVertIndex] : FColor::White;

		VertexPositions.Set(VertexID, Position);
		VertexInstanceNormals.Set(VertexInstanceID, TangentZ);
		VertexInstanceTangents.Set(VertexInstanceID, TangentX);
		VertexInstanceBinormalSigns.Set(VertexInstanceID, BinormalSign);
		VertexInstanceColors.Set(VertexInstanceID, FVector4f(FLinearColor(Color)));

		for (int32 UVChannelIndex = 0; UVChannelIndex < NumUVChannels; ++UVChannelIndex)
		{
			const FVector2f UV = InMeshData.Vertices.UVs[UVChannelIndex][InVertIndex];
			VertexInstanceUVs.Set(VertexInstanceID, UVChannelIndex, UV);
		}
	}

	const TArray<FStaticMaterial>& StaticMaterials = Context.StaticMesh->GetStaticMaterials();
	for (const Nanite::FMeshDataSection& Section : InMeshData.Sections)
	{
		const FPolygonGroupID PolygonGroupID = OutMesh.CreatePolygonGroup();
		const FName MaterialSlotName = StaticMaterials.IsValidIndex(Section.MaterialIndex) ?
			StaticMaterials[Section.MaterialIndex].ImportedMaterialSlotName : NAME_None;
		PolygonGroupMaterialSlotNames.Set(PolygonGroupID, MaterialSlotName);

		for (uint32 TriIndex = 0; TriIndex < Section.NumTriangles; ++TriIndex)
		{
			const FVertexInstanceID TriVertInstanceIDs[] = {
				FVertexInstanceID(InMeshData.TriangleIndices[Section.FirstIndex + TriIndex * 3 + 0]),
				FVertexInstanceID(InMeshData.TriangleIndices[Section.FirstIndex + TriIndex * 3 + 1]),
				FVertexInstanceID(InMeshData.TriangleIndices[Section.FirstIndex + TriIndex * 3 + 2])
			};

			OutMesh.CreateTriangle(PolygonGroupID, MakeConstArrayView(TriVertInstanceIDs, 3));
		}
	}
}

struct FRayTracingFallbackBuildContext
{
	TArray<uint32, TInlineAllocator<2>> MaxNumOfTriangles;
	TArray<FMeshDescription> MeshDescriptions;
	Nanite::FRayTracingFallbackBuildSettings Settings;

	int32 NumFallbackLODs() const
	{
		return MeshDescriptions.Num();
	}
};

static bool BuildNanite(
	FStaticMeshNaniteBuildContext& Context,
	FStaticMeshLODResources& LOD0Resources,
	FMeshDescription& LOD0MeshDescription,
	Nanite::FResources& NaniteResources,
	FRayTracingFallbackBuildContext& RayTracingFallbackBuildContext
)
{
	if (!ensure(Context.IsValid()))
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE( FStaticMeshBuilder::BuildNanite );
	
	// If applicable, recursively gather and build assembly references, and form their final hierarchy
	Nanite::FInputAssemblyData InputAssemblyData;
	if (Context.bIsAssembly && !InitNaniteAssemblyData(Context, InputAssemblyData))
	{
		return false;
	}

	// Build new vertex buffers
	Nanite::IBuilderModule::FInputMeshData InputMeshData;
	if (!InitNaniteBuildInput(Context, InputMeshData, LOD0Resources.SourceMeshBounds))
	{
		return false;
	}

	// We don't need to generate a fallback when using a high res source model. Regular static mesh build will handle it
	const bool bGenerateFallback = !Context.bHiResSourceModel;
	const bool bGenerateRayTracingFallback = RayTracingFallbackBuildContext.NumFallbackLODs() > 0;
	Nanite::IBuilderModule::FOutputMeshData FallbackMeshData;
	Nanite::IBuilderModule::FOutputMeshData RayTracingFallbackMeshData;
	
	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	if (!Context.Builder->Build(
			NaniteResources,
			InputMeshData,
			bGenerateFallback ? &FallbackMeshData : nullptr,
			bGenerateRayTracingFallback ? &RayTracingFallbackMeshData : nullptr,
			bGenerateRayTracingFallback ? &RayTracingFallbackBuildContext.Settings : nullptr,
			Context.Settings,
			&InputAssemblyData
	))
	{
		if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
		{
			return false;
		}

		UE_LOGFMT_NSLOC(LogStaticMesh, Warning, "StaticMesh", "NaniteHiResBuildError", "Failed to build Nanite for HiRes static mesh. See previous line(s) for details.");
		return false;
	}

	const FMeshBuildSettings& BuildSettings = Context.StaticMesh->GetSourceModel(0).BuildSettings;

	auto HasValidSections = [](Nanite::IBuilderModule::FOutputMeshData& MeshData)
	{
		bool bHasValidSections = false;
		for (const Nanite::FMeshDataSection& Section : MeshData.Sections)
		{
			if (Section.NumTriangles > 0)
			{
				bHasValidSections = true;
				break;
			}
		}
		return bHasValidSections;
	};

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	// Copy over the output data to the static mesh LOD data
	// Certain output LODs might be empty if the builder decided it wasn't needed (then remove these LODs again)
	// TODO: Is this ever the case with LOD 0 though?
	if (bGenerateFallback)
	{
		// If there are valid sections then copy over data to the LODResource
		if (HasValidSections(FallbackMeshData))
		{
			bool bNeeds32BitIndices = false;
			for (Nanite::FMeshDataSection& Section : FallbackMeshData.Sections)
			{
				bNeeds32BitIndices |= Section.MaxVertexIndex > MAX_uint16;
			}

			LOD0Resources.Sections = Nanite::BuildStaticMeshSections(FallbackMeshData.Sections);

			TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshBuilder::Build::BufferInit);

			FStaticMeshVertexBufferFlags StaticMeshVertexBufferFlags;
			StaticMeshVertexBufferFlags.bNeedsCPUAccess = true;
			StaticMeshVertexBufferFlags.bUseBackwardsCompatibleF16TruncUVs = BuildSettings.bUseBackwardsCompatibleF16TruncUVs;

			const FConstMeshBuildVertexView OutputMeshVertices = MakeConstMeshBuildVertexView(FallbackMeshData.Vertices);
			LOD0Resources.VertexBuffers.StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(BuildSettings.bUseHighPrecisionTangentBasis);
			LOD0Resources.VertexBuffers.StaticMeshVertexBuffer.SetUseFullPrecisionUVs(BuildSettings.bUseFullPrecisionUVs);
			LOD0Resources.VertexBuffers.StaticMeshVertexBuffer.Init(OutputMeshVertices, StaticMeshVertexBufferFlags);
			LOD0Resources.VertexBuffers.PositionVertexBuffer.Init(OutputMeshVertices);
			LOD0Resources.VertexBuffers.ColorVertexBuffer.Init(OutputMeshVertices);

			const EIndexBufferStride::Type IndexBufferStride = bNeeds32BitIndices ? EIndexBufferStride::Force32Bit : EIndexBufferStride::Force16Bit;
			LOD0Resources.IndexBuffer.SetIndices(FallbackMeshData.TriangleIndices, IndexBufferStride);

			BuildAllBufferOptimizations(LOD0Resources, BuildSettings, FallbackMeshData.TriangleIndices, bNeeds32BitIndices, OutputMeshVertices);

			// Fill out the mesh description for non-Nanite build/reduction
			BuildNaniteFallbackMeshDescription(Context, FallbackMeshData, LOD0MeshDescription);
		}
		else
		{
			// Initialize the mesh description as empty
			FStaticMeshAttributes(LOD0MeshDescription).Register();
		}
	}

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	if (bGenerateRayTracingFallback)
	{
		if (HasValidSections(RayTracingFallbackMeshData))
		{
			// Fill out the mesh description for ray tracing fallback build/reduction
			BuildNaniteFallbackMeshDescription(Context, RayTracingFallbackMeshData, RayTracingFallbackBuildContext.MeshDescriptions[0]);
		}
		else
		{
			// Initialize the mesh description as empty
			FStaticMeshAttributes(RayTracingFallbackBuildContext.MeshDescriptions[0]).Register();
		}
	}

	return true;
}

bool PrepareRayTracingFallbackBuild(
	const FStaticMeshNaniteBuildContext& NaniteBuildContext,
	FRayTracingFallbackBuildContext& OutBuildContext)
{
	const UStaticMesh* StaticMesh = NaniteBuildContext.StaticMesh;
	const ITargetPlatform* TargetPlatform = NaniteBuildContext.TargetPlatform;
	check(StaticMesh && TargetPlatform);

	if (StaticMesh->bSupportRayTracing && TargetPlatform->UsesRayTracing())
	{
		static const TConsoleVariableData<bool>* CVarRayTracingProxies = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.StaticMesh.RayTracingProxies"));
		static const TConsoleVariableData<int32>* CVarRayTracingProxiesLOD1MaxNumOfTriangles = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.StaticMesh.RayTracingProxies.LOD1MaxNumOfTriangles"));

		const FMeshRayTracingProxySettings& Settings = StaticMesh->GetRayTracingProxySettings();
		const bool bNeedsRayTracingProxy =
			(CVarRayTracingProxies && CVarRayTracingProxies->GetValueOnAnyThread())
			&& (Settings.bEnabled || ShouldGenerateRayTracingProxiesByDefault());

		if (bNeedsRayTracingProxy)
		{
			const uint32 LOD1MaxNumOfTriangles = CVarRayTracingProxiesLOD1MaxNumOfTriangles->GetValueOnAnyThread();

			// LOD0 is generated by Nanite build and is already simplified so no need to reduce
			OutBuildContext.MaxNumOfTriangles.Add(MAX_uint32);
			OutBuildContext.MaxNumOfTriangles.Add(LOD1MaxNumOfTriangles);

			OutBuildContext.MeshDescriptions.SetNum(OutBuildContext.MaxNumOfTriangles.Num());

			FMeshNaniteSettings NaniteSettings;
			NaniteSettings.FallbackTarget = Settings.FallbackTarget;
			NaniteSettings.FallbackPercentTriangles = Settings.FallbackPercentTriangles;
			NaniteSettings.FallbackRelativeError = Settings.FallbackRelativeError;
			const FMeshDescription* InputMeshDescription = NaniteBuildContext.SourceModel->GetCachedMeshDescription();
			check(InputMeshDescription);

			Nanite::CorrectFallbackSettings(NaniteSettings, InputMeshDescription->Triangles().Num(), NaniteBuildContext.bIsAssembly, /* bIsRayTracing */ true);

			OutBuildContext.Settings.FallbackPercentTriangles = NaniteSettings.FallbackPercentTriangles;
			OutBuildContext.Settings.FallbackRelativeError = NaniteSettings.FallbackRelativeError;

			// limit FoliageOverOcclusionBias since setting it to 1.0 removes all triangles and causes issues at runtime code paths.
			OutBuildContext.Settings.FoliageOverOcclusionBias = FMath::Min(Settings.FoliageOverOcclusionBias, 0.9f);

			return true;
		}
	}

	return false;
}

bool FStaticMeshBuilder::Build(FStaticMeshRenderData& StaticMeshRenderData, const FStaticMeshBuildParameters& BuildParameters)
{
	if (BuildParameters.TargetPlatform == nullptr)
	{
		UE_LOGF(LogStaticMeshBuilder, Error, "Provided FStaticMeshBuildParameters must have a valid TargetPlatform.");
		return false;
	}

	UStaticMesh* StaticMesh = BuildParameters.StaticMesh;
	const FStaticMeshLODGroup& LODGroup = BuildParameters.LODGroup;

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	if (!StaticMesh->IsMeshDescriptionValid(0))
	{
		//Warn the user that there is no mesh description data
		UE_LOGF(LogStaticMeshBuilder, Error, "Cannot find a valid mesh description to build the asset.");
		return false;
	}

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	if (StaticMeshRenderData.LODResources.Num() > 0)
	{
		//At this point the render data is suppose to be empty
		UE_LOGF(LogStaticMeshBuilder, Error, "Cannot build static mesh render data twice [%ls].", *StaticMesh->GetFullName());
		
		//Crash in debug
		checkSlow(StaticMeshRenderData.LODResources.Num() == 0);

		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshBuilder::Build);

	const int32 NumSourceModels = StaticMesh->GetNumSourceModels();
	StaticMeshRenderData.AllocateLODResources(NumSourceModels);

	FStaticMeshNaniteBuildContext NaniteBuildContext;
	const bool bBuildNanite = PrepareNaniteStaticMeshBuild(NaniteBuildContext, StaticMesh, BuildParameters.TargetPlatform);

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	FRayTracingFallbackBuildContext RayTracingFallbackBuildContext;
	bool bBuildRayTracingFallback = false;
	if (bBuildNanite)
	{
		bBuildRayTracingFallback = PrepareRayTracingFallbackBuild(NaniteBuildContext, RayTracingFallbackBuildContext);
	}

	int32 NumTasks = NumSourceModels;
	NumTasks += NaniteBuildContext.bHiResSourceModel ? 1 : 0;
	NumTasks += RayTracingFallbackBuildContext.NumFallbackLODs();
	FScopedSlowTask SlowTask(NumTasks, NSLOCTEXT("StaticMeshEditor", "StaticMeshBuilderBuild", "Building static mesh render data."));
	SlowTask.MakeDialog();

	FBoxSphereBounds::Builder MeshBoundsBuilder;

	const FMeshSectionInfoMap BeforeBuildSectionInfoMap = StaticMesh->GetSectionInfoMap();
	const FMeshSectionInfoMap BeforeBuildOriginalSectionInfoMap = StaticMesh->GetOriginalSectionInfoMap();

	TArray<FMeshDescription> MeshDescriptions;
	MeshDescriptions.SetNum(NumSourceModels);

	int32 NaniteBuiltLevels = 0;

	if (bBuildNanite)
	{
		SlowTask.EnterProgressFrame( 1 );

		Nanite::FResources& NaniteResources = *StaticMeshRenderData.NaniteResourcesPtr.Get();
		bool bBuildSuccess = BuildNanite(
			NaniteBuildContext,
			StaticMeshRenderData.LODResources[0],
			MeshDescriptions[0],
			NaniteResources,
			RayTracingFallbackBuildContext);

		if (bBuildSuccess)
		{
			FBoxSphereBounds NaniteBounds;
			NaniteBounds.Origin = FVector(NaniteResources.MeshBounds.Origin);
			NaniteBounds.BoxExtent = FVector(NaniteResources.MeshBounds.BoxExtent);
			NaniteBounds.SphereRadius = NaniteResources.MeshBounds.SphereRadius;
			MeshBoundsBuilder += NaniteBounds;

			if (!NaniteBuildContext.bHiResSourceModel)
			{
				// We don't need to build LOD 0 below if the Nanite build generated it
				++NaniteBuiltLevels;
			}
		}
	}

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	TFunction<void(const FMeshDescription&, const FMeshDescription&)> CheckReduction = [&](const FMeshDescription& InitMesh, const FMeshDescription& ReducedMesh)
	{
		FBox BBoxInitMesh = InitMesh.ComputeBoundingBox();
		double BBoxInitMeshSize = (BBoxInitMesh.Max - BBoxInitMesh.Min).Length();

		FBox BBoxReducedMesh = ReducedMesh.ComputeBoundingBox();
		double BBoxReducedMeshSize = (BBoxReducedMesh.Max - BBoxReducedMesh.Min).Length();

		constexpr double ThresholdForAbnormalGrowthOfBBox = UE_DOUBLE_SQRT_3; // the reduced mesh must stay in the bounding sphere 
		if (BBoxReducedMeshSize > BBoxInitMeshSize * ThresholdForAbnormalGrowthOfBBox)
		{
			UE_LOGF(LogStaticMeshBuilder, Warning, "The generation of LOD could have generated spikes on the mesh for %ls", *StaticMesh->GetName());
		}
	};

	// Build non-Nanite render data for each LOD
	for (int32 LodIndex = NaniteBuiltLevels; LodIndex < NumSourceModels; ++LodIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("FStaticMeshBuilder::Build LOD");
		SlowTask.EnterProgressFrame((LodIndex > 0 || !bBuildNanite) ? 1.0f : 0.0f);
		FScopedSlowTask BuildLODSlowTask(3);
		BuildLODSlowTask.EnterProgressFrame(1);

		FStaticMeshSourceModel& SrcModel = StaticMesh->GetSourceModel(LodIndex);

		// NOTE: Make a local copy on the stack, as build settings are used to generate the DDC key for static mesh, and
		// the mesh description helper might make changes to validate some settings
		FMeshBuildSettings LODBuildSettings = SrcModel.BuildSettings;

		float MaxDeviation = 0.0f;
		bool bIsMeshDescriptionValid = StaticMesh->CloneMeshDescription(LodIndex, MeshDescriptions[LodIndex]);
		bIsMeshDescriptionValid &= !MeshDescriptions[LodIndex].IsEmpty();
		FMeshDescriptionHelper MeshDescriptionHelper(&LODBuildSettings);

		FMeshReductionSettings ReductionSettings = LODGroup.GetSettings(SrcModel.ReductionSettings, LodIndex);

		// Make sure we do not reduce a non custom LOD by itself
		const int32 BaseReduceLodIndex = FMath::Clamp<int32>(ReductionSettings.BaseLODModel, 0, bIsMeshDescriptionValid ? LodIndex : LodIndex - 1);
		// Use simplifier if a reduction in triangles or verts has been requested.
		bool bUseReduction = StaticMesh->IsReductionActive(LodIndex);

		if (bIsMeshDescriptionValid)
		{
			MeshDescriptionHelper.SetupRenderMeshDescription(StaticMesh, MeshDescriptions[LodIndex], false, true);

			if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
			{
				return false;
			}

			//Make sure the cache is good before looking for the active reduction
			if (SrcModel.CacheMeshDescriptionTrianglesCount == MAX_uint32)
			{
				SrcModel.CacheMeshDescriptionTrianglesCount = static_cast<uint32>(MeshDescriptions[LodIndex].Triangles().Num());
			}
			if (SrcModel.CacheMeshDescriptionVerticesCount == MAX_uint32)
			{
				SrcModel.CacheMeshDescriptionVerticesCount = static_cast<uint32>(FStaticMeshOperations::GetUniqueVertexCount(MeshDescriptions[LodIndex], MeshDescriptionHelper.GetOverlappingCorners()));
			}
			//Get back the reduction status once we apply all build settings, vertex count can change depending on the build settings
			bUseReduction = StaticMesh->IsReductionActive(LodIndex);
		}
		else
		{
			if (bUseReduction)
			{
				// Initialize an empty mesh description that the reduce will fill
				FStaticMeshAttributes(MeshDescriptions[LodIndex]).Register();
			}
			else
			{
				//Duplicate the lodindex 0 we have a 100% reduction which is like a duplicate
				MeshDescriptions[LodIndex] = MeshDescriptions[BaseReduceLodIndex];
				//Set the overlapping threshold
				float ComparisonThreshold = StaticMesh->GetSourceModel(BaseReduceLodIndex).BuildSettings.bRemoveDegenerates ? THRESH_POINTS_ARE_SAME : 0.0f;
				MeshDescriptionHelper.FindOverlappingCorners(MeshDescriptions[LodIndex], ComparisonThreshold);

				if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
				{
					return false;
				}

				if (LodIndex > 0)
				{
					
					//Make sure the SectionInfoMap is taken from the Base RawMesh
					int32 SectionNumber = StaticMesh->GetOriginalSectionInfoMap().GetSectionNumber(BaseReduceLodIndex);
					for (int32 SectionIndex = 0; SectionIndex < SectionNumber; ++SectionIndex)
					{
						//Keep the old data if its valid
						bool bHasValidLODInfoMap = StaticMesh->GetSectionInfoMap().IsValidSection(LodIndex, SectionIndex);
						//Section material index have to be remap with the ReductionSettings.BaseLODModel SectionInfoMap to create
						//a valid new section info map for the reduced LOD.
						if (!bHasValidLODInfoMap && StaticMesh->GetSectionInfoMap().IsValidSection(BaseReduceLodIndex, SectionIndex))
						{
							//Copy the BaseLODModel section info to the reduce LODIndex.
							FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(BaseReduceLodIndex, SectionIndex);
							FMeshSectionInfo OriginalSectionInfo = StaticMesh->GetOriginalSectionInfoMap().Get(BaseReduceLodIndex, SectionIndex);
							StaticMesh->GetSectionInfoMap().Set(LodIndex, SectionIndex, SectionInfo);
							StaticMesh->GetOriginalSectionInfoMap().Set(LodIndex, SectionIndex, OriginalSectionInfo);
						}
					}
				}
			}

			if (LodIndex > 0)
			{
				LODBuildSettings = StaticMesh->GetSourceModel(BaseReduceLodIndex).BuildSettings;
			}
		}

		if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
		{
			return false;
		}

		// Reduce LODs
		if (bUseReduction)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("FStaticMeshBuilder::Build - Reduce LOD");
			
			float OverlappingThreshold = LODBuildSettings.bRemoveDegenerates ? THRESH_POINTS_ARE_SAME : 0.0f;
			FOverlappingCorners OverlappingCorners;
			FStaticMeshOperations::FindOverlappingCorners(OverlappingCorners, MeshDescriptions[BaseReduceLodIndex], OverlappingThreshold);

			if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
			{
				return false;
			}

			int32 OldSectionInfoMapCount = StaticMesh->GetSectionInfoMap().GetSectionNumber(LodIndex);

			if (LodIndex == BaseReduceLodIndex)
			{
				//When using LOD 0, we use a copy of the mesh description since reduce do not support inline reducing
				FMeshDescription BaseMeshDescription = MeshDescriptions[BaseReduceLodIndex];
				MeshDescriptionHelper.ReduceLOD(BaseMeshDescription, MeshDescriptions[LodIndex], ReductionSettings, OverlappingCorners, MaxDeviation);
				CheckReduction(BaseMeshDescription, MeshDescriptions[LodIndex]);
			}
			else
			{
				MeshDescriptionHelper.ReduceLOD(MeshDescriptions[BaseReduceLodIndex], MeshDescriptions[LodIndex], ReductionSettings, OverlappingCorners, MaxDeviation);
				CheckReduction(MeshDescriptions[BaseReduceLodIndex], MeshDescriptions[LodIndex]);
			}

			const TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = MeshDescriptions[LodIndex].PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName);
			const TPolygonGroupAttributesRef<FName> BasePolygonGroupImportedMaterialSlotNames = MeshDescriptions[BaseReduceLodIndex].PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName);
			// Recompute adjacency information. Since we change the vertices when we reduce
			MeshDescriptionHelper.FindOverlappingCorners(MeshDescriptions[LodIndex], OverlappingThreshold);
			
			//Make sure the static mesh SectionInfoMap is up to date with the new reduce LOD
			//We have to remap the material index with the ReductionSettings.BaseLODModel sectionInfoMap
			//Set the new SectionInfoMap for this reduced LOD base on the ReductionSettings.BaseLODModel SectionInfoMap
			TArray<int32> BaseUniqueMaterialIndexes;
			//Find all unique Material in used order
			for (const FPolygonGroupID PolygonGroupID : MeshDescriptions[BaseReduceLodIndex].PolygonGroups().GetElementIDs())
			{
				int32 MaterialIndex = StaticMesh->GetMaterialIndexFromImportedMaterialSlotName(BasePolygonGroupImportedMaterialSlotNames[PolygonGroupID]);
				if (MaterialIndex == INDEX_NONE)
				{
					MaterialIndex = PolygonGroupID.GetValue();
				}
				BaseUniqueMaterialIndexes.AddUnique(MaterialIndex);
			}
			TArray<int32> UniqueMaterialIndex;
			//Find all unique Material in used order
			for (const FPolygonGroupID PolygonGroupID : MeshDescriptions[LodIndex].PolygonGroups().GetElementIDs())
			{
				int32 MaterialIndex = StaticMesh->GetMaterialIndexFromImportedMaterialSlotName(PolygonGroupImportedMaterialSlotNames[PolygonGroupID]);
				if (MaterialIndex == INDEX_NONE)
				{
					MaterialIndex = PolygonGroupID.GetValue();
				}
				UniqueMaterialIndex.AddUnique(MaterialIndex);
			}

			//If the reduce did not output the same number of section use the base LOD sectionInfoMap
			bool bIsOldMappingInvalid = OldSectionInfoMapCount != MeshDescriptions[LodIndex].PolygonGroups().Num();

			bool bValidBaseSectionInfoMap = BeforeBuildSectionInfoMap.GetSectionNumber(BaseReduceLodIndex) > 0;
			//All used material represent a different section
			for (int32 SectionIndex = 0; SectionIndex < UniqueMaterialIndex.Num(); ++SectionIndex)
			{
				//Keep the old data
				bool bHasValidLODInfoMap = !bIsOldMappingInvalid && BeforeBuildSectionInfoMap.IsValidSection(LodIndex, SectionIndex);
				//Section material index have to be remap with the ReductionSettings.BaseLODModel SectionInfoMap to create
				//a valid new section info map for the reduced LOD.

				//Find the base LOD section using this material
				if (!bHasValidLODInfoMap)
				{
					bool bSectionInfoSet = false;
					if (bValidBaseSectionInfoMap)
					{
						for (int32 BaseSectionIndex = 0; BaseSectionIndex < BaseUniqueMaterialIndexes.Num(); ++BaseSectionIndex)
						{
							if (UniqueMaterialIndex[SectionIndex] == BaseUniqueMaterialIndexes[BaseSectionIndex])
							{
								//Copy the base sectionInfoMap
								FMeshSectionInfo SectionInfo = BeforeBuildSectionInfoMap.Get(BaseReduceLodIndex, BaseSectionIndex);
								FMeshSectionInfo OriginalSectionInfo = BeforeBuildOriginalSectionInfoMap.Get(BaseReduceLodIndex, BaseSectionIndex);
								StaticMesh->GetSectionInfoMap().Set(LodIndex, SectionIndex, SectionInfo);
								StaticMesh->GetOriginalSectionInfoMap().Set(LodIndex, BaseSectionIndex, OriginalSectionInfo);
								bSectionInfoSet = true;
								break;
							}
						}
					}

					if (!bSectionInfoSet)
					{
						//Just set the default section info in case we did not found any match with the Base Lod
						FMeshSectionInfo SectionInfo;
						SectionInfo.MaterialIndex = SectionIndex;
						StaticMesh->GetSectionInfoMap().Set(LodIndex, SectionIndex, SectionInfo);
						StaticMesh->GetOriginalSectionInfoMap().Set(LodIndex, SectionIndex, SectionInfo);
					}
				}
			}
		}
		BuildLODSlowTask.EnterProgressFrame(1);
		const FPolygonGroupArray& PolygonGroups = MeshDescriptions[LodIndex].PolygonGroups();

		FStaticMeshLODResources& StaticMeshLOD = StaticMeshRenderData.LODResources[LodIndex];
		StaticMeshLOD.MaxDeviation = MaxDeviation;

		// Build new vertex buffers
		FMeshBuildVertexData BuildVertexData;

		StaticMeshLOD.Sections.Empty(PolygonGroups.Num());
		TArray<int32> RemapVerts; //Because we will remove MeshVertex that are redundant, we need a remap
									//Render data Wedge map is only set for LOD 0???

		TArray<int32>& WedgeMap = StaticMeshLOD.WedgeMap;
		WedgeMap.Reset();

		// Prepare the PerSectionIndices array so we can optimize the index buffer for the GPU
		TArray<TArray<uint32> > PerSectionIndices;
		PerSectionIndices.AddDefaulted(MeshDescriptions[LodIndex].PolygonGroups().Num());

		if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
		{
			return false;
		}

		// Build the vertex and index buffer
		UE::Private::StaticMeshBuilder::BuildVertexBuffer(
			StaticMesh,
			MeshDescriptions[LodIndex],
			LODBuildSettings,
			WedgeMap,
			StaticMeshLOD.Sections,
			PerSectionIndices,
			BuildVertexData,
			MeshDescriptionHelper.GetOverlappingCorners(),
			RemapVerts,
			StaticMeshLOD.SourceMeshBounds,
			true /* bNeedTangents */,
			true /* bNeedWedgeMap */
		);

		MeshBoundsBuilder += StaticMeshLOD.SourceMeshBounds;

		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = MeshDescriptions[LodIndex].VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
		const uint32 NumTextureCoord = VertexInstanceUVs.IsValid() ? VertexInstanceUVs.GetNumChannels() : 0;

		// Only the render data and vertex buffers will be used from now on unless we have more than one source models
		// This will help with memory usage for Nanite Mesh by releasing memory before doing the build
		if (NumSourceModels == 1)
		{
			MeshDescriptions.Empty();
		}

		// Concatenate the per-section index buffers.
		TArray<uint32> CombinedIndices;
		bool bNeeds32BitIndices = false;
		UE::Private::StaticMeshBuilder::BuildCombinedSectionIndices(PerSectionIndices, StaticMeshLOD.Sections, CombinedIndices, bNeeds32BitIndices);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshBuilder::Build::BufferInit);

			FConstMeshBuildVertexView ConstVertexView = MakeConstMeshBuildVertexView(BuildVertexData);

			StaticMeshLOD.VertexBuffers.StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(LODBuildSettings.bUseHighPrecisionTangentBasis);
			StaticMeshLOD.VertexBuffers.StaticMeshVertexBuffer.SetUseFullPrecisionUVs(LODBuildSettings.bUseFullPrecisionUVs);
			FStaticMeshVertexBufferFlags StaticMeshVertexBufferFlags;
			StaticMeshVertexBufferFlags.bNeedsCPUAccess = true;
			StaticMeshVertexBufferFlags.bUseBackwardsCompatibleF16TruncUVs = LODBuildSettings.bUseBackwardsCompatibleF16TruncUVs;
			StaticMeshLOD.VertexBuffers.StaticMeshVertexBuffer.Init(ConstVertexView, StaticMeshVertexBufferFlags);
			StaticMeshLOD.VertexBuffers.PositionVertexBuffer.Init(ConstVertexView);
			StaticMeshLOD.VertexBuffers.ColorVertexBuffer.Init(ConstVertexView);

			const EIndexBufferStride::Type IndexBufferStride = bNeeds32BitIndices ? EIndexBufferStride::Force32Bit : EIndexBufferStride::Force16Bit;
			StaticMeshLOD.IndexBuffer.SetIndices(CombinedIndices, IndexBufferStride);

			// post-process the index buffer
			BuildLODSlowTask.EnterProgressFrame(1);
			BuildAllBufferOptimizations(StaticMeshLOD, LODBuildSettings, CombinedIndices, bNeeds32BitIndices, ConstVertexView);
		}

	} //End of LOD for loop

	// Update the render data bounds
	StaticMeshRenderData.Bounds = MeshBoundsBuilder;

	if (bBuildRayTracingFallback)
	{
		const int32 NumRayTracingLODs = RayTracingFallbackBuildContext.NumFallbackLODs();

		checkf(!StaticMeshRenderData.RayTracingProxy, TEXT("RayTracingProxy expected to be null. Was the static mesh ray tracing representation already initialized?"));
		StaticMeshRenderData.RayTracingProxy = new FStaticMeshRayTracingProxy();
		StaticMeshRenderData.RayTracingProxy->bUsingRenderingLODs = false;

		FStaticMeshRayTracingProxyLODArray& RayTracingLODs = StaticMeshRenderData.RayTracingProxy->LODs;
		RayTracingLODs.Reserve(NumRayTracingLODs);

		StaticMeshRenderData.RayTracingProxy->LODVertexFactories = new FStaticMeshVertexFactoriesArray;
		FStaticMeshVertexFactoriesArray& RayTracingLODVertexFactories = *StaticMeshRenderData.RayTracingProxy->LODVertexFactories;
		RayTracingLODVertexFactories.Reserve(NumRayTracingLODs);

		for (int32 LodIndex = 0; LodIndex < NumRayTracingLODs; ++LodIndex)
		{
			if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
			{
				return false;
			}

			TRACE_CPUPROFILER_EVENT_SCOPE_STR("FStaticMeshBuilder::Build Ray Tracing Proxy");
			SlowTask.EnterProgressFrame(1);
			FScopedSlowTask BuildLODSlowTask(3);
			BuildLODSlowTask.EnterProgressFrame(1);

			const int32 BaseReduceLodIndex = 0;
			// NOTE: Make a local copy on the stack, as build settings are used to generate the DDC key for static mesh, and
			// the mesh description helper might make changes to validate some settings
			FMeshBuildSettings LODBuildSettings = StaticMesh->GetSourceModel(BaseReduceLodIndex).BuildSettings;

			FMeshDescriptionHelper MeshDescriptionHelper(&LODBuildSettings);

			FMeshReductionSettings ReductionSettings;
			ReductionSettings.MaxNumOfTriangles = RayTracingFallbackBuildContext.MaxNumOfTriangles[LodIndex];
			ReductionSettings.TerminationCriterion = EStaticMeshReductionTerimationCriterion::Triangles;

			TArray<FMeshDescription>& RayTracingMeshDescriptions = RayTracingFallbackBuildContext.MeshDescriptions;
			const bool bUseReduction = ReductionSettings.PercentTriangles < 1.f || ReductionSettings.MaxNumOfTriangles < (uint32)RayTracingMeshDescriptions[BaseReduceLodIndex].Triangles().Num();

			// Reduce LODs
			if (bUseReduction)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("FStaticMeshBuilder::Build - Reduce Ray Tracing LOD");

				FStaticMeshAttributes(RayTracingMeshDescriptions[LodIndex]).Register();

				const float OverlappingThreshold = LODBuildSettings.bRemoveDegenerates ? THRESH_POINTS_ARE_SAME : 0.0f;
				FOverlappingCorners OverlappingCorners;
				FStaticMeshOperations::FindOverlappingCorners(OverlappingCorners, RayTracingMeshDescriptions[BaseReduceLodIndex], OverlappingThreshold);

				float MaxDeviation = 0.f;
				if (LodIndex == BaseReduceLodIndex)
				{
					//When using LOD 0, we use a copy of the mesh description since reduce do not support inline reducing
					FMeshDescription BaseMeshDescription = RayTracingMeshDescriptions[BaseReduceLodIndex];
					MeshDescriptionHelper.ReduceLOD(BaseMeshDescription, RayTracingMeshDescriptions[LodIndex], ReductionSettings, OverlappingCorners, MaxDeviation);
					CheckReduction(BaseMeshDescription, RayTracingMeshDescriptions[LodIndex]);
				}
				else
				{
					MeshDescriptionHelper.ReduceLOD(RayTracingMeshDescriptions[BaseReduceLodIndex], RayTracingMeshDescriptions[LodIndex], ReductionSettings, OverlappingCorners, MaxDeviation);
					CheckReduction(RayTracingMeshDescriptions[BaseReduceLodIndex], RayTracingMeshDescriptions[LodIndex]);
				}

				// Recompute adjacency information. Since we change the vertices when we reduce
				MeshDescriptionHelper.FindOverlappingCorners(RayTracingMeshDescriptions[LodIndex], OverlappingThreshold);
			}
			else
			{
				// Nanite build has already generated LOD0 mesh description
				if (LodIndex > BaseReduceLodIndex)
				{
					RayTracingMeshDescriptions[LodIndex] = RayTracingMeshDescriptions[BaseReduceLodIndex];
				}

				//Set the overlapping threshold
				const float ComparisonThreshold = LODBuildSettings.bRemoveDegenerates ? THRESH_POINTS_ARE_SAME : 0.0f;
				MeshDescriptionHelper.FindOverlappingCorners(RayTracingMeshDescriptions[LodIndex], ComparisonThreshold);
			}

			BuildLODSlowTask.EnterProgressFrame(1);
			const FPolygonGroupArray& PolygonGroups = RayTracingMeshDescriptions[LodIndex].PolygonGroups();

			// Build new vertex buffers
			FMeshBuildVertexData BuildVertexData;
			FStaticMeshSectionArray Sections;
			TArray<int32> RemapVerts; //Because we will remove MeshVertex that are redundant, we need a remap
			TArray<int32> WedgeMap;
			TArray<TArray<uint32>> PerSectionIndices;
			FBoxSphereBounds LODBounds;

			Sections.Empty(PolygonGroups.Num());
			PerSectionIndices.AddDefaulted(PolygonGroups.Num());

			// Build the vertex and index buffer
			UE::Private::StaticMeshBuilder::BuildVertexBuffer(
				StaticMesh,
				RayTracingMeshDescriptions[LodIndex],
				LODBuildSettings,
				WedgeMap,
				Sections,
				PerSectionIndices,
				BuildVertexData,
				MeshDescriptionHelper.GetOverlappingCorners(),
				RemapVerts,
				LODBounds,
				true /* bNeedTangents */,
				true /* bNeedWedgeMap */
			);

			if (LodIndex == NumRayTracingLODs - 1)
			{
				RayTracingMeshDescriptions.Empty();
			}

			// Concatenate the per-section index buffers.
			TArray<uint32> CombinedIndices;
			bool bNeeds32BitIndices = false;
			UE::Private::StaticMeshBuilder::BuildCombinedSectionIndices(PerSectionIndices, Sections, CombinedIndices, bNeeds32BitIndices);

			bool bHasValidSections = false;
			for (const FStaticMeshSection& Section : Sections)
			{
				if (Section.NumTriangles > 0)
				{
					bHasValidSections = true;
					break;
				}
			}

			// If there are valid sections then copy over data to the RayTracingProxy
			if (bHasValidSections)
			{
				RayTracingLODVertexFactories.Add(FStaticMeshVertexFactories(GMaxRHIFeatureLevel));

				FStaticMeshRayTracingProxyLOD* RayTracingLOD = new FStaticMeshRayTracingProxyLOD;
				RayTracingLODs.Add(RayTracingLOD);
				RayTracingLOD->Sections = new FStaticMeshSectionArray;
				RayTracingLOD->VertexBuffers = new FStaticMeshVertexBuffers;
				RayTracingLOD->IndexBuffer = new FRawStaticIndexBuffer;
				RayTracingLOD->bOwnsBuffers = true;
				RayTracingLOD->bOwnsRayTracingGeometry = true;
				RayTracingLOD->RayTracingGeometry = new FRayTracingGeometry;

				RayTracingLOD->Sections->Empty(Sections.Num());
				for (const FStaticMeshSection& Section : Sections)
				{
					RayTracingLOD->Sections->Add(Section);
				}

				TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshBuilder::Build::BufferInit);

				FStaticMeshVertexBufferFlags StaticMeshVertexBufferFlags;
				StaticMeshVertexBufferFlags.bNeedsCPUAccess = true;
				StaticMeshVertexBufferFlags.bUseBackwardsCompatibleF16TruncUVs = LODBuildSettings.bUseBackwardsCompatibleF16TruncUVs;

				const FConstMeshBuildVertexView OutputMeshVertices = MakeConstMeshBuildVertexView(BuildVertexData);
				RayTracingLOD->VertexBuffers->StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(LODBuildSettings.bUseHighPrecisionTangentBasis);
				RayTracingLOD->VertexBuffers->StaticMeshVertexBuffer.SetUseFullPrecisionUVs(LODBuildSettings.bUseFullPrecisionUVs);
				RayTracingLOD->VertexBuffers->StaticMeshVertexBuffer.Init(OutputMeshVertices, StaticMeshVertexBufferFlags);
				RayTracingLOD->VertexBuffers->PositionVertexBuffer.Init(OutputMeshVertices);
				RayTracingLOD->VertexBuffers->ColorVertexBuffer.Init(OutputMeshVertices);

				// Why is the 'bNeeds32BitIndices' used from the original index buffer? Is that needed?
				const EIndexBufferStride::Type IndexBufferStride = bNeeds32BitIndices ? EIndexBufferStride::Force32Bit : EIndexBufferStride::Force16Bit;
				RayTracingLOD->IndexBuffer->SetIndices(CombinedIndices, IndexBufferStride);

				BuildLODSlowTask.EnterProgressFrame(1);
			}
		}
	}

	if (StaticMesh->bSupportRayTracing && BuildParameters.TargetPlatform->UsesRayTracing())
	{
		if (!StaticMeshRenderData.RayTracingProxy)
		{
			StaticMeshRenderData.InitializeRayTracingRepresentationFromRenderingLODs();
		}
		else
		{
			check(!StaticMeshRenderData.RayTracingProxy->bUsingRenderingLODs);
		}
	}
	
	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	return true;
}

bool FStaticMeshBuilder::BuildMeshVertexPositions(
	UStaticMesh* StaticMesh,
	TArray<uint32>& BuiltIndices,
	TArray<FVector3f>& BuiltVertices,
	FStaticMeshSectionArray& Sections)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshBuilder::BuildMeshVertexPositions);

	FStaticMeshSourceModel& SourceModel = StaticMesh->IsHiResMeshDescriptionValid() ? StaticMesh->GetHiResSourceModel() : StaticMesh->GetSourceModel(0);
	if (!SourceModel.IsMeshDescriptionValid())
	{
		//Warn the user that there is no mesh description data
		UE_LOGF(LogStaticMeshBuilder, Error, "Cannot find a valid mesh description to build the asset.");
		return false;
	}

	FMeshDescription MeshDescription;
	const bool bIsMeshDescriptionValid = SourceModel.CloneMeshDescription(MeshDescription);

	if (MeshDescription.IsEmpty())
	{
		UE_LOGF(LogStaticMeshBuilder, Error, "Cannot build the asset from an empty mesh description.");
		return false;
	}

	FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;

	FMeshDescriptionHelper MeshDescriptionHelper(&BuildSettings);
	MeshDescriptionHelper.SetupRenderMeshDescription(StaticMesh, MeshDescription, false, false);
	
	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	const FPolygonGroupArray& PolygonGroups = MeshDescription.PolygonGroups();

	// Build new vertex buffers
	FMeshBuildVertexData BuildVertexData;

	Sections.Empty(PolygonGroups.Num());

	TArray<int32> RemapVerts; //Because we will remove MeshVertex that are redundant, we need a remap
	//Render data Wedge map is only set for LOD 0???

	TArray<int32> WedgeMap;

	// Prepare the PerSectionIndices array so we can optimize the index buffer for the GPU
	TArray<TArray<uint32>> PerSectionIndices;
	PerSectionIndices.AddDefaulted(MeshDescription.PolygonGroups().Num());

	FBoxSphereBounds LODBounds;

	// Build the vertex and index buffer
	UE::Private::StaticMeshBuilder::BuildVertexBuffer(
		StaticMesh,
		MeshDescription,
		BuildSettings,
		WedgeMap,
		Sections,
		PerSectionIndices,
		BuildVertexData,
		MeshDescriptionHelper.GetOverlappingCorners(),
		RemapVerts,
		LODBounds,
		false /* bNeedTangents */,
		false /* bNeedWedgeMap */
	);

	BuiltVertices = BuildVertexData.Position;

	// Release MeshDescription memory since we don't need it anymore
	MeshDescription.Empty();

	// Concatenate the per-section index buffers.
	bool bNeeds32BitIndices = false;
	UE::Private::StaticMeshBuilder::BuildCombinedSectionIndices(PerSectionIndices, Sections, BuiltIndices, bNeeds32BitIndices);

	// Apply section remapping
	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); SectionIndex++)
	{
		Sections[SectionIndex].MaterialIndex = StaticMesh->GetSectionInfoMap().Get(0, SectionIndex).MaterialIndex;
	}

	return true;
}

namespace UE::Private::StaticMeshBuilder
{

struct FPendingVertex
{
	FVector3f Position;
	FVector3f TangentX;
	FVector3f TangentY;
	FVector3f TangentZ;
	float CurveRadius;
	FColor Color;
	TStaticArray<FVector2f, MAX_STATIC_TEXCOORDS> UVs;
};

bool AreVerticesEqual(const FPendingVertex& Vertex, const FMeshBuildVertexData& VertexData, int32 CompareVertex, float ComparisonThreshold)
{
	if (!Vertex.Position.Equals(VertexData.Position[CompareVertex], ComparisonThreshold))
	{
		return false;
	}

	// Test TangentZ first, often X and Y are zero
	if (!NormalsEqual(Vertex.TangentZ, VertexData.TangentZ[CompareVertex]))
	{
		return false;
	}

	if (!NormalsEqual(Vertex.TangentX, VertexData.TangentX[CompareVertex]))
	{
		return false;
	}

	if (!NormalsEqual(Vertex.TangentY, VertexData.TangentY[CompareVertex]))
	{
		return false;
	}

	if (VertexData.CurveRadius.Num() > 0)
	{
		if (Vertex.CurveRadius != VertexData.CurveRadius[CompareVertex])
		{
			return false;
		}
	}

	if (VertexData.Color.Num() > 0)
	{
		if (Vertex.Color != VertexData.Color[CompareVertex])
		{
			return false;
		}
	}

	// UVs
	for (int32 UVIndex = 0; UVIndex < VertexData.UVs.Num(); UVIndex++)
	{
		if (!UVsEqual(Vertex.UVs[UVIndex], VertexData.UVs[UVIndex][CompareVertex]))
		{
			return false;
		}
	}

	return true;
}


void BuildVertexBuffer(
	UStaticMesh* StaticMesh,
	const FMeshDescription& MeshDescription,
	const FMeshBuildSettings& BuildSettings,
	TArray<int32>& OutWedgeMap,
	FStaticMeshSectionArray& OutSections,
	TArray<TArray<uint32>>& OutPerSectionIndices,
	FMeshBuildVertexData& BuildVertexData,
	const FOverlappingCorners& OverlappingCorners,
	TArray<int32>& RemapVerts,
	FBoxSphereBounds& MeshBounds,
	bool bNeedTangents,
	bool bNeedWedgeMap
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BuildVertexBuffer);

	const int32 NumVertexInstances = MeshDescription.VertexInstances().GetArraySize();
	const bool bCacheOptimize = (NumVertexInstances < 100000 * 3);

	FBounds3f Bounds;
	bool bBoundsSet = false;

	FStaticMeshConstAttributes Attributes(MeshDescription);

	TPolygonGroupAttributesConstRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesConstRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

	const bool bHasColors = VertexInstanceColors.IsValid();
	bool bValidColors = false;
	const int32 NumTextureCoord = VertexInstanceUVs.IsValid() ? VertexInstanceUVs.GetNumChannels() : 0;
	const FVector3f BuildScale(BuildSettings.BuildScale3D);

	// No curve radius for meshes
	const bool bValidCurveRadius = false;

	// set up vertex buffer elements
	BuildVertexData.Position.Reserve(NumVertexInstances);
	BuildVertexData.TangentX.Reserve(NumVertexInstances);
	BuildVertexData.TangentY.Reserve(NumVertexInstances);
	BuildVertexData.TangentZ.Reserve(NumVertexInstances);
	BuildVertexData.UVs.SetNum(NumTextureCoord);
	for (int32 TexCoord = 0; TexCoord < NumTextureCoord; ++TexCoord)
	{
		BuildVertexData.UVs[TexCoord].Reserve(NumVertexInstances);
	}

	TMap<FPolygonGroupID, int32> PolygonGroupToSectionIndex;

	for (const FPolygonGroupID PolygonGroupID : MeshDescription.PolygonGroups().GetElementIDs())
	{
		int32& SectionIndex = PolygonGroupToSectionIndex.FindOrAdd(PolygonGroupID);
		SectionIndex = OutSections.Add(FStaticMeshSection());
		FStaticMeshSection& StaticMeshSection = OutSections[SectionIndex];
		StaticMeshSection.MaterialIndex = StaticMesh->GetMaterialIndexFromImportedMaterialSlotName(PolygonGroupImportedMaterialSlotNames[PolygonGroupID]);
		if (StaticMeshSection.MaterialIndex == INDEX_NONE)
		{
			StaticMeshSection.MaterialIndex = PolygonGroupID.GetValue();
		}
	}

	int32 ReserveIndicesCount = MeshDescription.Triangles().Num() * 3;

	// Fill the remap array
	{
		RemapVerts.AddZeroed(ReserveIndicesCount);
		for (int32& RemapIndex : RemapVerts)
		{
			RemapIndex = INDEX_NONE;
		}
	}

	// Initialize the wedge map array tracking correspondence between wedge index and rendering vertex index
	OutWedgeMap.Reset();
	if (bNeedWedgeMap)
	{
		OutWedgeMap.AddZeroed(ReserveIndicesCount);
	}

	float VertexComparisonThreshold = BuildSettings.bRemoveDegenerates ? THRESH_POINTS_ARE_SAME : 0.0f;

	bool bUseLegacyTangentScaling = StaticMesh->GetLegacyTangentScaling();

	int32 WedgeIndex = 0;
	for (const FTriangleID TriangleID : MeshDescription.Triangles().GetElementIDs())
	{
		const FPolygonGroupID PolygonGroupID = MeshDescription.GetTrianglePolygonGroup(TriangleID);
		
		const int32* FoundSectionIndex = PolygonGroupToSectionIndex.Find(PolygonGroupID);
		const int32 SectionIndex = 
			ensureMsgf(FoundSectionIndex, TEXT("Static mesh builder found mesh description triangle with invalid polygon group ID not in the mesh description polygon groups list: %d"), PolygonGroupID.GetValue()) 
			? *FoundSectionIndex : 0;
		TArray<uint32>& SectionIndices = OutPerSectionIndices[SectionIndex];

		TArrayView<const FVertexID> VertexIDs = MeshDescription.GetTriangleVertices(TriangleID);

		FVector3f CornerPositions[3];
		for (int32 TriVert = 0; TriVert < 3; ++TriVert)
		{
			CornerPositions[TriVert] = VertexPositions[VertexIDs[TriVert]];
		}
		FOverlappingThresholds OverlappingThresholds;
		OverlappingThresholds.ThresholdPosition = VertexComparisonThreshold;
		// Don't process degenerate triangles.
		if (PointsEqual(CornerPositions[0], CornerPositions[1], OverlappingThresholds)
			|| PointsEqual(CornerPositions[0], CornerPositions[2], OverlappingThresholds)
			|| PointsEqual(CornerPositions[1], CornerPositions[2], OverlappingThresholds))
		{
			WedgeIndex += 3;
			continue;
		}

		TArrayView<const FVertexInstanceID> VertexInstanceIDs = MeshDescription.GetTriangleVertexInstances(TriangleID);
		for (int32 TriVert = 0; TriVert < 3; ++TriVert, ++WedgeIndex)
		{
			const FVertexInstanceID VertexInstanceID = VertexInstanceIDs[TriVert];
			const FVector3f& VertexPosition = CornerPositions[TriVert];
			const FVector3f& VertexInstanceNormal = VertexInstanceNormals[VertexInstanceID];
			const FVector3f& VertexInstanceTangent = VertexInstanceTangents[VertexInstanceID];
			const float VertexInstanceBinormalSign = VertexInstanceBinormalSigns[VertexInstanceID];

			FPendingVertex PendingVertex;

			PendingVertex.Position = VertexPosition;
			PendingVertex.TangentX = VertexInstanceTangent;
			PendingVertex.TangentY = (VertexInstanceNormal ^ VertexInstanceTangent) * VertexInstanceBinormalSign;
			PendingVertex.TangentZ = VertexInstanceNormal;
			
			ScaleStaticMeshVertex(
				PendingVertex.Position,
				PendingVertex.TangentX,
				PendingVertex.TangentY,
				PendingVertex.TangentZ,
				BuildScale,
				bNeedTangents,
				bUseLegacyTangentScaling
			);

			FColor VertexColor = FColor::White;
			if (bHasColors)
			{
				const FVector4f& VertexInstanceColor = VertexInstanceColors[VertexInstanceID];
				const FLinearColor LinearColor(VertexInstanceColor);
				VertexColor = LinearColor.ToFColor(true);
			}

			PendingVertex.Color = VertexColor;

			for (int32 UVIndex = 0; UVIndex < NumTextureCoord; ++UVIndex)
			{
				PendingVertex.UVs[UVIndex] = VertexInstanceUVs.Get(VertexInstanceID, UVIndex);
			}

			int32 Index = INDEX_NONE;

			// Never add duplicated vertex instance
			// Use WedgeIndex since OverlappingCorners has been built based on that
			{
				const TArray<int32>& DupVerts = OverlappingCorners.FindIfOverlapping(WedgeIndex);
				for (int32 k = 0; k < DupVerts.Num(); k++)
				{
					if (DupVerts[k] >= WedgeIndex)
					{
						break;
					}
					int32 Location = RemapVerts.IsValidIndex(DupVerts[k]) ? RemapVerts[DupVerts[k]] : INDEX_NONE;
					if (Location != INDEX_NONE && AreVerticesEqual(PendingVertex, BuildVertexData, Location, VertexComparisonThreshold))
					{
						Index = Location;
						break;
					}
				}
			}

			if (Index == INDEX_NONE)
			{
				Index = BuildVertexData.Position.Emplace(PendingVertex.Position);
				
				BuildVertexData.TangentX.Emplace(PendingVertex.TangentX);
				BuildVertexData.TangentY.Emplace(PendingVertex.TangentY);
				BuildVertexData.TangentZ.Emplace(PendingVertex.TangentZ);
				
				if (bHasColors)
				{
					if (PendingVertex.Color != FColor::White)
					{
						bValidColors = true;
					}

					if (BuildVertexData.Color.Num() == 0 && bValidColors)
					{
						// First occurrence of a non fully opaque white color means we allocate output space,
						// and then set all previously encountered vertex colors to be opaque white.
						BuildVertexData.Color.Reserve(NumVertexInstances);
						BuildVertexData.Color.SetNumUninitialized(BuildVertexData.Position.Num() - 1);
						for (int32 ColorIndex = 0; ColorIndex < BuildVertexData.Color.Num(); ++ColorIndex)
						{
							BuildVertexData.Color[ColorIndex] = FColor::White;
						}
					}

					if (bValidColors)
					{
						BuildVertexData.Color.Emplace(PendingVertex.Color);
					}
				}

				for (int32 UVIndex = 0; UVIndex < NumTextureCoord; ++UVIndex)
				{
					BuildVertexData.UVs[UVIndex].Emplace(VertexInstanceUVs.Get(VertexInstanceID, UVIndex));
				}

				// We are already processing all vertices, so we may as well compute the bounding box here
				// instead of yet another loop over the vertices at a later point.
				Bounds += PendingVertex.Position;
				bBoundsSet = true;
			}

				RemapVerts[WedgeIndex] = Index;

			if (bNeedWedgeMap)
			{
				OutWedgeMap[WedgeIndex] = Index;
			}

			SectionIndices.Add(Index);
		}
	}

	if (!bBoundsSet)
	{
		// There were no verts that contribute to bounds, so we'll just set a bounds of 0,0,0 to avoid calculating NaNs for Origin, BoxExtent, and SphereRadius below
		Bounds = FVector3f(0.f, 0.f, 0.f);
	}

	// Calculate the bounding sphere, using the center of the bounding box as the origin.
	FVector3f Center = Bounds.GetCenter();
	float RadiusSqr = 0.0f;
	for (int32 VertexIndex = 0; VertexIndex < BuildVertexData.Position.Num(); VertexIndex++)
	{
		RadiusSqr = FMath::Max(RadiusSqr, (BuildVertexData.Position[VertexIndex] - Center).SizeSquared());
	}

	MeshBounds.Origin = FVector(Center);
	MeshBounds.BoxExtent = FVector(Bounds.GetExtent());
	MeshBounds.SphereRadius = FMath::Sqrt(RadiusSqr);

	// Optimize before setting the buffer
	if (bCacheOptimize)
	{
		BuildOptimizationHelper::CacheOptimizeVertexAndIndexBuffer(BuildVertexData, OutPerSectionIndices, OutWedgeMap);
		//check(OutWedgeMap.Num() == MeshDescription->VertexInstances().Num());
	}

	RemapVerts.Empty();
}

/**
 * Utility function used inside FStaticMeshBuilder::Build() per-LOD loop to populate
 * the Sections in a FStaticMeshLODResources from PerSectionIndices, as well as
 * concatenate all section indices into CombinedIndicesOut.
 * Returned bNeeds32BitIndicesOut indicates whether max vert index is larger than max int16
 */
void BuildCombinedSectionIndices(
	const TArray<TArray<uint32>>& PerSectionIndices,
	FStaticMeshSectionArray& SectionsOut,
	TArray<uint32>& CombinedIndicesOut,
	bool& bNeeds32BitIndicesOut)
{
	bNeeds32BitIndicesOut = false;
	for (int32 SectionIndex = 0; SectionIndex < SectionsOut.Num(); SectionIndex++)
	{
		FStaticMeshSection& Section = SectionsOut[SectionIndex];
		const TArray<uint32>& SectionIndices = PerSectionIndices[SectionIndex];
		Section.FirstIndex = 0;
		Section.NumTriangles = 0;
		Section.MinVertexIndex = 0;
		Section.MaxVertexIndex = 0;

		if (SectionIndices.Num())
		{
			Section.FirstIndex = CombinedIndicesOut.Num();
			Section.NumTriangles = SectionIndices.Num() / 3;

			CombinedIndicesOut.AddUninitialized(SectionIndices.Num());
			uint32* DestPtr = &CombinedIndicesOut[Section.FirstIndex];
			uint32 const* SrcPtr = SectionIndices.GetData();

			Section.MinVertexIndex = *SrcPtr;
			Section.MaxVertexIndex = *SrcPtr;

			for (int32 Index = 0; Index < SectionIndices.Num(); Index++)
			{
				uint32 VertIndex = *SrcPtr++;

				bNeeds32BitIndicesOut |= (VertIndex > MAX_uint16);
				Section.MinVertexIndex = FMath::Min<uint32>(VertIndex, Section.MinVertexIndex);
				Section.MaxVertexIndex = FMath::Max<uint32>(VertIndex, Section.MaxVertexIndex);
				*DestPtr++ = VertIndex;
			}
		}
	}
}

} // namespace UE::Private::StaticMeshBuilder

void BuildAllBufferOptimizations(FStaticMeshLODResources& StaticMeshLOD, const FMeshBuildSettings& LODBuildSettings, TArray< uint32 >& IndexBuffer, bool bNeeds32BitIndices, const FConstMeshBuildVertexView& BuildVertices)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BuildAllBufferOptimizations);

	static const TConsoleVariableData<int32>* CVarDepthOnlyIndexBuffers = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportDepthOnlyIndexBuffers"));
	static const TConsoleVariableData<int32>* CVarReversedIndexBuffers = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportReversedIndexBuffers"));

	const bool bDepthOnlyIndexBuffers = !!CVarDepthOnlyIndexBuffers->GetValueOnAnyThread();
	const bool bReversedIndexBuffers = !!CVarReversedIndexBuffers->GetValueOnAnyThread();

	if (StaticMeshLOD.AdditionalIndexBuffers == nullptr)
	{
		StaticMeshLOD.AdditionalIndexBuffers = new FAdditionalStaticMeshIndexBuffers();
	}

	const EIndexBufferStride::Type IndexBufferStride = bNeeds32BitIndices ? EIndexBufferStride::Force32Bit : EIndexBufferStride::Force16Bit;

	// Build the reversed index buffer.
	if (LODBuildSettings.bBuildReversedIndexBuffer && bReversedIndexBuffers)
	{
		TArray<uint32> InversedIndices;
		const int32 IndexCount = IndexBuffer.Num();
		InversedIndices.AddUninitialized(IndexCount);

		for (int32 SectionIndex = 0; SectionIndex < StaticMeshLOD.Sections.Num(); ++SectionIndex)
		{
			const FStaticMeshSection& SectionInfo = StaticMeshLOD.Sections[SectionIndex];
			const int32 SectionIndexCount = SectionInfo.NumTriangles * 3;

			for (int32 i = 0; i < SectionIndexCount; ++i)
			{
				InversedIndices[SectionInfo.FirstIndex + i] = IndexBuffer[SectionInfo.FirstIndex + SectionIndexCount - 1 - i];
			}
		}
		StaticMeshLOD.AdditionalIndexBuffers->ReversedIndexBuffer.SetIndices(InversedIndices, IndexBufferStride);
	}

	// Build the depth-only index buffer.
	TArray<uint32> DepthOnlyIndices;
	if (bDepthOnlyIndexBuffers)
	{
		BuildOptimizationHelper::BuildDepthOnlyIndexBuffer(
			DepthOnlyIndices,
			BuildVertices,
			IndexBuffer,
			StaticMeshLOD.Sections
		);

		if (DepthOnlyIndices.Num() < 50000 * 3)
		{
			BuildOptimizationThirdParty::CacheOptimizeIndexBuffer(DepthOnlyIndices);
		}

		StaticMeshLOD.DepthOnlyIndexBuffer.SetIndices(DepthOnlyIndices, IndexBufferStride);
	}

	// Build the inversed depth only index buffer.
	if (LODBuildSettings.bBuildReversedIndexBuffer && bReversedIndexBuffers && bDepthOnlyIndexBuffers)
	{
		TArray<uint32> ReversedDepthOnlyIndices;
		const int32 IndexCount = DepthOnlyIndices.Num();
		ReversedDepthOnlyIndices.AddUninitialized(IndexCount);
		for (int32 i = 0; i < IndexCount; ++i)
		{
			ReversedDepthOnlyIndices[i] = DepthOnlyIndices[IndexCount - 1 - i];
		}
		StaticMeshLOD.AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.SetIndices(ReversedDepthOnlyIndices, IndexBufferStride);
	}

	// Build a list of wireframe edges in the static mesh.
	{
		TArray<BuildOptimizationHelper::FMeshEdge> Edges;
		TArray<uint32> WireframeIndices;

		BuildOptimizationHelper::FMeshEdgeBuilder(IndexBuffer, BuildVertices, Edges).FindEdges();
		WireframeIndices.Empty(2 * Edges.Num());
		for (int32 EdgeIndex = 0; EdgeIndex < Edges.Num(); EdgeIndex++)
		{
			BuildOptimizationHelper::FMeshEdge&	Edge = Edges[EdgeIndex];
			WireframeIndices.Add(Edge.Vertices[0]);
			WireframeIndices.Add(Edge.Vertices[1]);
		}
		StaticMeshLOD.AdditionalIndexBuffers->WireframeIndexBuffer.SetIndices(WireframeIndices, IndexBufferStride);
	}
}

