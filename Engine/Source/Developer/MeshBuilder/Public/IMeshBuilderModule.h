// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"

// TODO: Remove these dependencies when removing the deprecated functions below
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"

struct FStaticMeshSection;
class FStaticMeshSectionArray;

class IMeshBuilderModule : public IModuleInterface
{
public:
	static inline IMeshBuilderModule& GetForPlatform(const ITargetPlatform* TargetPlatform)
	{
		check(TargetPlatform);
		return FModuleManager::LoadModuleChecked<IMeshBuilderModule>(TargetPlatform->GetMeshBuilderModuleName());
	}

	static inline IMeshBuilderModule& GetForRunningPlatform()
	{
		const ITargetPlatform* TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		return GetForPlatform(TargetPlatform);
	}

	virtual void AppendToDDCKey(FString& DDCKey, bool bSkeletal) { }

	virtual bool BuildMesh(class FStaticMeshRenderData& OutRenderData, const struct FStaticMeshBuildParameters& BuildParameters) = 0;

	UE_DEPRECATED(5.5, "Use FStaticMeshBuildParameters instead.")
	virtual bool BuildMesh(class FStaticMeshRenderData& OutRenderData, class UObject* Mesh, const class FStaticMeshLODGroup& LODGroup, bool bAllowNanite) final
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Mesh))
		{
			return BuildMesh(OutRenderData, FStaticMeshBuildParameters(StaticMesh, nullptr, LODGroup));
		}
		return false;
	}

	virtual bool BuildMeshVertexPositions(class UObject* StaticMesh, TArray<uint32>& Indices, TArray<FVector3f>& Vertices, FStaticMeshSectionArray& Sections) = 0;

	UE_DEPRECATED(5.6, "Use the overload that takes an FSkeletalMeshRenderData instead.")
	virtual bool BuildSkeletalMesh(const struct FSkeletalMeshBuildParameters& SkeletalMeshBuildParameters) final
	{
		FSkeletalMeshRenderData* RenderData = SkeletalMeshBuildParameters.SkeletalMesh->GetResourceForRendering();
		check(RenderData != nullptr);
		return BuildSkeletalMesh(*RenderData, SkeletalMeshBuildParameters);
	}
	
	virtual bool BuildSkeletalMesh(class FSkeletalMeshRenderData& OutRenderData, const struct FSkeletalMeshBuildParameters& SkeletalMeshBuildParameters) = 0;

	virtual void PostBuildSkeletalMesh(class FSkeletalMeshRenderData* SkeletalMeshRenderData, class USkinnedAsset* SkinnedAsset) { }
};
