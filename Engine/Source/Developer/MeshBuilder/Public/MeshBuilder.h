// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Math/UnrealMathSSE.h"

#define UE_API MESHBUILDER_API

class FStaticMeshLODGroup;
class FStaticMeshRenderData;
class FStaticMeshSectionArray;
class UStaticMesh;
struct FStaticMeshBuildParameters;
struct FSkeletalMeshBuildParameters;
class FSkeletalMeshRenderData;
struct FMeshBuildVertexData;

/**
 * Abstract class which is the base class of all builder.
 * All share code to build some render data should be found inside this class
 */
class FMeshBuilder
{
public:
	UE_API FMeshBuilder();
	
	/**
	 * Build function should be override and is the starting point for static mesh builders
	 */
	virtual bool Build(FStaticMeshRenderData& OutRenderData, const FStaticMeshBuildParameters& BuildParameters) = 0;

	UE_DEPRECATED(5.5, "Use FStaticMeshBuildParameters instead.")
	UE_API virtual bool Build(
		FStaticMeshRenderData& OutRenderData,
		UStaticMesh* StaticMesh,
		const FStaticMeshLODGroup& LODGroup,
		bool bAllowNanite) final;

	virtual bool BuildMeshVertexPositions(
		UStaticMesh* StaticMesh,
		TArray<uint32>& Indices,
		TArray<FVector3f>& Vertices,
		FStaticMeshSectionArray& Sections) = 0;

	/**
	 * Build function should be override and is the starting point for skeletal mesh builders
	 */
	virtual bool Build(FSkeletalMeshRenderData& OutRenderData, const FSkeletalMeshBuildParameters& SkeletalMeshBuildParameters) = 0;

	UE_DEPRECATED(5.6, "Use the overload that takes FSkeletalMeshRenderData instead")
	UE_API virtual bool Build(const FSkeletalMeshBuildParameters& SkeletalMeshBuildParameters) final;

private:

};

#undef UE_API
