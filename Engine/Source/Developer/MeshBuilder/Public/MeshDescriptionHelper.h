// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "OverlappingCorners.h"

#define UE_API MESHBUILDER_API

DECLARE_LOG_CATEGORY_EXTERN(LogMeshBuilder, Log, All);

class UObject;
struct FMeshDescription;
struct FMeshBuildSettings;
struct FVertexInstanceID;
struct FMeshReductionSettings;

class FMeshDescriptionHelper
{
public:

	UE_API FMeshDescriptionHelper(FMeshBuildSettings* InBuildSettings);

	//Build a render mesh description with the BuildSettings. This will update the RenderMeshDescription in place
	UE_API void SetupRenderMeshDescription(UObject* Owner, FMeshDescription& RenderMeshDescription, bool bForNanite, bool bNeedTangents);

	UE_API void ReduceLOD(const FMeshDescription& BaseMesh, FMeshDescription& DestMesh, const struct FMeshReductionSettings& ReductionSettings, const FOverlappingCorners& InOverlappingCorners, float &OutMaxDeviation);

	UE_API void FindOverlappingCorners(const FMeshDescription& MeshDescription, float ComparisonThreshold);

	const FOverlappingCorners& GetOverlappingCorners() const { return OverlappingCorners; }

private:

	//////////////////////////////////////////////////////////////////////////
	//PRIVATE function declarations

	//////////////////////////////////////////////////////////////////////////
	//PRIVATE class members

	FMeshBuildSettings* BuildSettings;
	FOverlappingCorners OverlappingCorners;

	
	//////////////////////////////////////////////////////////////////////////
	//INLINE small helper use to optimize search and compare

	/**
	* Smoothing group interpretation helper structure.
	*/
	struct FFanFace
	{
		int32 FaceIndex;
		int32 LinkedVertexIndex;
		bool bFilled;
		bool bBlendTangents;
		bool bBlendNormals;
	};

};

#undef UE_API
