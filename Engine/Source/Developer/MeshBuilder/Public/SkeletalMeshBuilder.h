// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshBuilder.h"

#define UE_API MESHBUILDER_API

DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalMeshBuilder, Log, All);

class UStaticMesh;
class FStaticMeshRenderData;
class FStaticMeshLODGroup;
class USkeletalMesh;
class FSkeletalMeshRenderData;

class FSkeletalMeshBuilder : public FMeshBuilder
{
public:
	UE_API FSkeletalMeshBuilder();

	//No support for static mesh build in this class
	virtual bool Build(FStaticMeshRenderData& OutRenderData, const struct FStaticMeshBuildParameters& BuildParameters) override
	{
		bool No_Support_For_StaticMesh_Build_In_FSkeletalMeshBuilder_Class = false;
		check(No_Support_For_StaticMesh_Build_In_FSkeletalMeshBuilder_Class);
		return false;
	}

	virtual bool BuildMeshVertexPositions(
		UStaticMesh* StaticMesh,
		TArray<uint32>& Indices,
		TArray<FVector3f>& Vertices,
		FStaticMeshSectionArray& Sections) override
	{
		bool No_Support_For_StaticMesh_Build_In_FSkeletalMeshBuilder_Class = false;
		check(No_Support_For_StaticMesh_Build_In_FSkeletalMeshBuilder_Class);
		return false;
	}
	
	UE_API virtual bool Build(FSkeletalMeshRenderData& OutRenderData, const struct FSkeletalMeshBuildParameters& SkeletalMeshBuildParameters) override;

	virtual ~FSkeletalMeshBuilder() = default;

private:

};

#undef UE_API
