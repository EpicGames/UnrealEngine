// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshBuilder.h"

#define UE_API MESHBUILDER_API

DECLARE_LOG_CATEGORY_EXTERN(LogStaticMeshBuilder, Log, All);

class ITargetPlatform;
class UStaticMesh;
class FStaticMeshRenderData;
class FStaticMeshLODGroup;
struct FOverlappingCorners;
struct FMeshDescription;
struct FMeshBuildSettings;

class FStaticMeshBuilder : public FMeshBuilder
{
public:
	UE_API FStaticMeshBuilder();
	virtual ~FStaticMeshBuilder() = default;

	UE_API virtual bool Build(FStaticMeshRenderData& OutRenderData, const FStaticMeshBuildParameters& BuildParameters) override;

	//No support for skeletal mesh build in this class
	virtual bool Build(FSkeletalMeshRenderData& OutRenderData, const struct FSkeletalMeshBuildParameters& SkeletalMeshBuildParameters) override
	{
		bool No_Support_For_SkeletalMesh_Build_In_FStaticMeshBuilder_Class = false;
		check(No_Support_For_SkeletalMesh_Build_In_FStaticMeshBuilder_Class);
		return false;
	}

	UE_API virtual bool BuildMeshVertexPositions(
		UStaticMesh* StaticMesh,
		TArray<uint32>& Indices,
		TArray<FVector3f>& Vertices,
		FStaticMeshSectionArray& Sections) override;

private:

	UE_API void OnBuildRenderMeshStart(class UStaticMesh* StaticMesh, const bool bInvalidateLighting);
	UE_API void OnBuildRenderMeshFinish(class UStaticMesh* StaticMesh, const bool bRebuildBoundsAndCollision);

	/** Used to refresh all components in the scene that may be using a mesh we're editing */
	TSharedPtr<class FStaticMeshComponentRecreateRenderStateContext> RecreateRenderStateContext;
};

namespace UE::Private::StaticMeshBuilder
{

	MESHBUILDER_API void BuildVertexBuffer(
		UStaticMesh* StaticMesh,
		const FMeshDescription& MeshDescription,
		const FMeshBuildSettings& BuildSettings,
		TArray<int32>& OutWedgeMap,
		FStaticMeshSectionArray& OutSections,
		TArray<TArray<uint32>>& OutPerSectionIndices,
		FMeshBuildVertexData& BuildVertexData,
		const FOverlappingCorners& OverlappingCorners,
		TArray<int32>& RemapVerts,
		FBoxSphereBounds& MeshBounds,
		bool bNeedTangents,
		bool bNeedWedgeMap
	);

	MESHBUILDER_API void BuildCombinedSectionIndices(
		const TArray<TArray<uint32>>& PerSectionIndices,
		FStaticMeshSectionArray& SectionsOut,
		TArray<uint32>& CombinedIndicesOut,
		bool& bNeeds32BitIndicesOut);
}

#undef UE_API
